{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Animation Warping",
	"Description": "Framework for animation and pose warping. This plugin includes Stride, Orientation, and Slope Warping alongside the Root Motion Delta animation attribute.",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "AnimationWarpingRuntime",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "AnimationWarpingEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault"
		}
	],
	"Plugins": [
		{
			"Name": "AnimationModifierLibrary",
			"Enabled": true
		}
	]
}