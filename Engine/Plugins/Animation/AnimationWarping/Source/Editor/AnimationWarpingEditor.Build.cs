// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AnimationWarpingEditor : ModuleRules
	{
		public AnimationWarpingEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AnimationCore",
				"AnimationWarpingRuntime",
                "AnimGraph",
				"AnimGraphRuntime",
				"AnimationModifiers",
				"AnimationBlueprintLibrary",
				"AnimationModifierLibrary",
				"Core",
                "CoreUObject",
                "Engine",
            });

			PrivateDependencyModuleNames.AddRange(
			new string[]
			{
                "SlateCore",
            });

            if (Target.bBuildEditor == true)
            {
                PrivateDependencyModuleNames.AddRange(
                    new string[]
                    {
						"BlueprintGraph",
						"EditorFramework",
						"Kismet",
                        "UnrealEd",
                    }
                );
            }
        }
	}
}
