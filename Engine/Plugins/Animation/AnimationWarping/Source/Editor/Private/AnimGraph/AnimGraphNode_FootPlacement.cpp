// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraph/AnimGraphNode_FootPlacement.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_FootPlacement

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_FootPlacement)

#define LOCTEXT_NAMESPACE "AnimGraphNode_FootPlacement"

UAnimGraphNode_FootPlacement::UAnimGraphNode_FootPlacement(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UAnimGraphNode_FootPlacement::GetControllerDescription() const
{
	return LOCTEXT("FootPlacement", "Foot Placement");
}

FText UAnimGraphNode_FootPlacement::GetTooltipText() const
{
	return LOCTEXT("FootPlacementTooltip", "Foot Placement.");
}

FLinearColor UAnimGraphNode_FootPlacement::GetNodeTitleColor() const
{
	return FLinearColor(FColor(153.f, 0.f, 0.f));
}

FText UAnimGraphNode_FootPlacement::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return GetControllerDescription();
}

#undef LOCTEXT_NAMESPACE
