// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraph/AnimGraphNode_OffsetRootBone.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "DetailLayoutBuilder.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_OffsetRootBone)

#define LOCTEXT_NAMESPACE "AnimationWarping"

UAnimGraphNode_OffsetRootBone::UAnimGraphNode_OffsetRootBone(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

FText UAnimGraphNode_OffsetRootBone::GetTooltipText() const
{
	return LOCTEXT("OffsetRootBoneTooltip", "Applies an offset to the root bone");
}

FText UAnimGraphNode_OffsetRootBone::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("OffsetRootBone", "Offset Root Bone");
}

FLinearColor UAnimGraphNode_OffsetRootBone::GetNodeTitleColor() const
{
	return FLinearColor(FColor(153.f, 0.f, 0.f));
}

void UAnimGraphNode_OffsetRootBone::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	Super::CustomizePinData(Pin, SourcePropertyName, ArrayIndex);

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_OffsetRootBone, TranslationDelta))
	{
		Pin->bHidden = (Node.EvaluationMode == EWarpingEvaluationMode::Graph);
	}

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_OffsetRootBone, RotationDelta))
	{
		Pin->bHidden = (Node.EvaluationMode == EWarpingEvaluationMode::Graph);
	}
}

void UAnimGraphNode_OffsetRootBone::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	Super::CustomizeDetails(DetailBuilder);

	TSharedRef<IPropertyHandle> NodeHandle = DetailBuilder.GetProperty(FName(TEXT("Node")), GetClass());

	if (Node.EvaluationMode == EWarpingEvaluationMode::Graph)
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_OffsetRootBone, TranslationDelta)));
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_OffsetRootBone, RotationDelta)));
	}
}

void UAnimGraphNode_OffsetRootBone::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	Super::PostEditChangeProperty(PropertyChangedEvent);

	bool bRequiresNodeReconstruct = false;
	FProperty* ChangedProperty = PropertyChangedEvent.Property;

	if (ChangedProperty)
	{
		if ((ChangedProperty->GetFName() == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_OffsetRootBone, EvaluationMode)))
		{
			FScopedTransaction Transaction(LOCTEXT("OffsetRootBone_ChangeEvaluationMode", "Offset Root Bone Change Evaluation Mode"));
			Modify();

			// Break links to pins going away
			for (int32 PinIndex = 0; PinIndex < Pins.Num(); ++PinIndex)
			{
				UEdGraphPin* Pin = Pins[PinIndex];
				if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_OffsetRootBone, TranslationDelta))
				{
					if (Node.EvaluationMode == EWarpingEvaluationMode::Graph)
					{
						Pin->BreakAllPinLinks();
					}
				}
				else if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_OffsetRootBone, RotationDelta))
				{
					if (Node.EvaluationMode == EWarpingEvaluationMode::Graph)
					{
						Pin->BreakAllPinLinks();
					}
				}
			}

			bRequiresNodeReconstruct = true;
		}
	}

	if (bRequiresNodeReconstruct)
	{
		ReconstructNode();
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}
}

void UAnimGraphNode_OffsetRootBone::GetInputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	OutAttributes.Add(UE::Anim::IAnimRootMotionProvider::AttributeName);
}

void UAnimGraphNode_OffsetRootBone::GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	OutAttributes.Add(UE::Anim::IAnimRootMotionProvider::AttributeName);
}

#undef LOCTEXT_NAMESPACE
