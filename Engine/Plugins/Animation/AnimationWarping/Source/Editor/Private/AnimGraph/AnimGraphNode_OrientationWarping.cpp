// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraph/AnimGraphNode_OrientationWarping.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_OrientationWarping)

#define LOCTEXT_NAMESPACE "AnimationWarping"

UAnimGraphNode_OrientationWarping::UAnimGraphNode_OrientationWarping(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

FText UAnimGraphNode_OrientationWarping::GetControllerDescription() const
{
	return LOCTEXT("OrientationWarping", "Orientation Warping");
}

FText UAnimGraphNode_OrientationWarping::GetTooltipText() const
{
	return LOCTEXT("OrientationWarpingTooltip", "Rotates the root and lower body by the specified angle, while counter rotating the upper body to maintain the forward facing direction.");
}

FText UAnimGraphNode_OrientationWarping::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return GetControllerDescription();
}

FLinearColor UAnimGraphNode_OrientationWarping::GetNodeTitleColor() const
{
	return FLinearColor(FColor(153.f, 0.f, 0.f));
}

void UAnimGraphNode_OrientationWarping::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	Super::CustomizePinData(Pin, SourcePropertyName, ArrayIndex);

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_OrientationWarping, OrientationAngle))
	{
		Pin->bHidden = (Node.Mode == EWarpingEvaluationMode::Graph);
	}

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_OrientationWarping, LocomotionAngle))
	{
		Pin->bHidden = (Node.Mode == EWarpingEvaluationMode::Manual);
	}
	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_OrientationWarping, LocomotionDirection))
   	{
   		Pin->bHidden = (Node.Mode == EWarpingEvaluationMode::Manual);
   	}

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_OrientationWarping, LocomotionAngleDeltaThreshold))
	{
		Pin->bHidden = (Node.Mode == EWarpingEvaluationMode::Manual);
	}

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_OrientationWarping, WarpingSpace))
	{
		Pin->bHidden = (Node.WarpingSpace == EOrientationWarpingSpace::CustomTransform);
	}
}

void UAnimGraphNode_OrientationWarping::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	Super::CustomizeDetails(DetailBuilder);

	DetailBuilder.SortCategories([](const TMap<FName, IDetailCategoryBuilder*>& CategoryMap)
	{
		for (const TPair<FName, IDetailCategoryBuilder*>& Pair : CategoryMap)
		{
			int32 SortOrder = Pair.Value->GetSortOrder();
			const FName CategoryName = Pair.Key;

			if (CategoryName == "Evaluation")
			{
				SortOrder += 1;
			}
			else if (CategoryName == "Settings")
			{
				SortOrder += 2;
			}
			else if (CategoryName == "Debug")
			{
				SortOrder += 3;
			}
			else
			{
				const int32 ValueSortOrder = Pair.Value->GetSortOrder();
				if (ValueSortOrder >= SortOrder && ValueSortOrder < SortOrder + 10)
				{
					SortOrder += 10;
				}
				else
				{
					continue;
				}
			}

			Pair.Value->SetSortOrder(SortOrder);
		}
	});

	TSharedRef<IPropertyHandle> NodeHandle = DetailBuilder.GetProperty(FName(TEXT("Node")), GetClass());

	if (Node.Mode == EWarpingEvaluationMode::Graph)
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_OrientationWarping, OrientationAngle)));
	}

	if (Node.Mode == EWarpingEvaluationMode::Manual)
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_OrientationWarping, LocomotionAngle)));
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_OrientationWarping, LocomotionDirection)));
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_OrientationWarping, LocomotionAngleDeltaThreshold)));
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_OrientationWarping, MinRootMotionSpeedThreshold)));
	}
	
	if (Node.WarpingSpace != EOrientationWarpingSpace::CustomTransform)
   	{
    	DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_OrientationWarping, WarpingSpaceTransform)));
    }
}

void UAnimGraphNode_OrientationWarping::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	Super::PostEditChangeProperty(PropertyChangedEvent);

	bool bRequiresNodeReconstruct = false;
	FProperty* ChangedProperty = PropertyChangedEvent.Property;

	if (ChangedProperty)
	{
		// Evaluation mode
		if (ChangedProperty->GetFName() == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_OrientationWarping, Mode))
		{
			FScopedTransaction Transaction(LOCTEXT("ChangeEvaluationMode", "Change Evaluation Mode"));
			Modify();

			// Break links to pins going away
			for (int32 PinIndex = 0; PinIndex < Pins.Num(); ++PinIndex)
			{
				UEdGraphPin* Pin = Pins[PinIndex];
				if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_OrientationWarping, OrientationAngle))
				{
					if (Node.Mode == EWarpingEvaluationMode::Graph)
					{
						Pin->BreakAllPinLinks();
					}
				}
				else if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_OrientationWarping, LocomotionAngle))
				{
					if (Node.Mode == EWarpingEvaluationMode::Manual)
					{
						Pin->BreakAllPinLinks();
					}
				}
				else if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_OrientationWarping, LocomotionDirection))
				{
					if (Node.Mode == EWarpingEvaluationMode::Manual)
					{
						Pin->BreakAllPinLinks();
					}
				}
				else if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_OrientationWarping, LocomotionAngleDeltaThreshold))
				{
					if (Node.Mode == EWarpingEvaluationMode::Manual)
					{
						Pin->BreakAllPinLinks();
					}
				}
				else if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_OrientationWarping, MinRootMotionSpeedThreshold))
				{
					if (Node.Mode == EWarpingEvaluationMode::Manual)
					{
						Pin->BreakAllPinLinks();
					}
				}
			}

			bRequiresNodeReconstruct = true;
		}
	}

	if (bRequiresNodeReconstruct)
	{
		ReconstructNode();
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}
}

void UAnimGraphNode_OrientationWarping::GetInputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	if (Node.Mode == EWarpingEvaluationMode::Graph)
	{
		OutAttributes.Add(UE::Anim::IAnimRootMotionProvider::AttributeName);
	}
}

void UAnimGraphNode_OrientationWarping::GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	if (Node.Mode == EWarpingEvaluationMode::Graph)
	{
		OutAttributes.Add(UE::Anim::IAnimRootMotionProvider::AttributeName);
	}
}

void UAnimGraphNode_OrientationWarping::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	auto HasInvalidBoneName = [](const FName& BoneName)
	{
		return BoneName == NAME_None;
	};

	auto HasInvalidBoneIndex = [&](const FName& BoneName)
	{
		return ForSkeleton && ForSkeleton->GetReferenceSkeleton().FindBoneIndex(BoneName) == INDEX_NONE;
	};

	auto InvalidBoneNameMessage = [&](const FName& BoneName)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("BoneName"), FText::FromName(BoneName));
		const FText Message = FText::Format(NSLOCTEXT("OrientationWarping", "Invalid{BoneName}BoneName", "@@ - {BoneName} bone not found in Skeleton"), Args);
		MessageLog.Warning(*Message.ToString(), this);
	};

	auto InvalidBoneIndexMessage = [&](const FName& BoneName)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("BoneName"), FText::FromName(BoneName));
		const FText Message = FText::Format(NSLOCTEXT("OrientationWarping", "Invalid{BoneName}BoneInSkeleton", "@@ - {BoneName} bone definition is required"), Args);
		MessageLog.Warning(*Message.ToString(), this);
	};

	if (Node.RotationAxis == EAxis::None)
	{
		MessageLog.Warning(*NSLOCTEXT("OrientationWarping", "InvalidRotationAxis", "@@ - Rotation Axis choice of X, Y, or Z is required").ToString(), this);
	}

	if (Node.SpineBones.IsEmpty())
	{
		MessageLog.Warning(*NSLOCTEXT("OrientationWarping", "InvalidSpineBones", "@@ - Spine bone definitions are required").ToString(), this);
	}
	else
	{
		for (const auto& Bone : Node.SpineBones)
		{
			if (HasInvalidBoneName(Bone.BoneName))
			{
				InvalidBoneIndexMessage("Spine");
			}
			else if (HasInvalidBoneIndex(Bone.BoneName))
			{
				InvalidBoneNameMessage(Bone.BoneName);
			}
		}
	}

	if (HasInvalidBoneName(Node.IKFootRootBone.BoneName))
	{
		InvalidBoneIndexMessage("IK Foot Root");
	}
	else if (HasInvalidBoneIndex(Node.IKFootRootBone.BoneName))
	{
		InvalidBoneNameMessage(Node.IKFootRootBone.BoneName);
	}

	if (Node.SpineBones.IsEmpty())
	{
		MessageLog.Warning(*NSLOCTEXT("OrientationWarping", "InvalidIKFootBones", "@@ - IK Foot bone definitions are required").ToString(), this);
	}
	else
	{
		for (const auto& Bone : Node.IKFootBones)
		{
			if (HasInvalidBoneName(Bone.BoneName))
			{
				InvalidBoneIndexMessage("IK Foot");
			}
			else if (HasInvalidBoneIndex(Bone.BoneName))
			{
				InvalidBoneNameMessage(Bone.BoneName);
			}
		}
	}

	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);
}

#undef LOCTEXT_NAMESPACE
