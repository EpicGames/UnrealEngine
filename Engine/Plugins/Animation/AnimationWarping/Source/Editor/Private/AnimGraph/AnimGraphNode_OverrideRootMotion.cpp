// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraph/AnimGraphNode_OverrideRootMotion.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Kismet2/BlueprintEditorUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_OverrideRootMotion)

#define LOCTEXT_NAMESPACE "AnimationWarping"

UAnimGraphNode_OverrideRootMotion::UAnimGraphNode_OverrideRootMotion(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

FText UAnimGraphNode_OverrideRootMotion::GetTooltipText() const
{
	return LOCTEXT("OverrideRootMotionTooltip", "Overrides the root motion attribute");
}

FText UAnimGraphNode_OverrideRootMotion::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("OverrideRootMotion", "Override Root Motion");
}

FLinearColor UAnimGraphNode_OverrideRootMotion::GetNodeTitleColor() const
{
	return FLinearColor(FColor(153.f, 0.f, 0.f));
}

void UAnimGraphNode_OverrideRootMotion::GetInputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	OutAttributes.Add(UE::Anim::IAnimRootMotionProvider::AttributeName);
}

void UAnimGraphNode_OverrideRootMotion::GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	OutAttributes.Add(UE::Anim::IAnimRootMotionProvider::AttributeName);
}

#undef LOCTEXT_NAMESPACE
