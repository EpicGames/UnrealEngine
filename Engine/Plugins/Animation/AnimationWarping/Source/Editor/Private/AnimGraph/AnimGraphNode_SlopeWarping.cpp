// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraph/AnimGraphNode_SlopeWarping.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_SlopeWarping)

#define LOCTEXT_NAMESPACE "AnimationWarping"

UAnimGraphNode_SlopeWarping::UAnimGraphNode_SlopeWarping(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

FText UAnimGraphNode_SlopeWarping::GetControllerDescription() const
{
	return LOCTEXT("SlopeWarping", "Slope Warping");
}

FText UAnimGraphNode_SlopeWarping::GetTooltipText() const
{
	return LOCTEXT("SlopeWarpingTooltip", "Warps the feet to match the floor normal.");
}

FText UAnimGraphNode_SlopeWarping::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return GetControllerDescription();
}

FLinearColor UAnimGraphNode_SlopeWarping::GetNodeTitleColor() const
{
	return FLinearColor(FColor(153.f, 0.f, 0.f));
}

#undef LOCTEXT_NAMESPACE
