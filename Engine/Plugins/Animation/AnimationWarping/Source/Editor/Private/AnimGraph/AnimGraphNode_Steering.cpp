// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraph/AnimGraphNode_Steering.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_Steering)

#define LOCTEXT_NAMESPACE "AnimationWarping"

UAnimGraphNode_Steering::UAnimGraphNode_Steering(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

FText UAnimGraphNode_Steering::GetControllerDescription() const
{
	return LOCTEXT("Steering", "Steering");
}

FText UAnimGraphNode_Steering::GetTooltipText() const
{
	return LOCTEXT("SteeringTooltip", "Adds root motion rotation to the Root Motion Attribute, in order to face the Target Orientation.");
}

FText UAnimGraphNode_Steering::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return GetControllerDescription();
}

FLinearColor UAnimGraphNode_Steering::GetNodeTitleColor() const
{
	return FLinearColor(FColor(153.f, 0.f, 0.f));
}

void UAnimGraphNode_Steering::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	Super::CustomizePinData(Pin, SourcePropertyName, ArrayIndex);
}

void UAnimGraphNode_Steering::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	Super::CustomizeDetails(DetailBuilder);
}

void UAnimGraphNode_Steering::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UAnimGraphNode_Steering::GetInputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	OutAttributes.Add(UE::Anim::IAnimRootMotionProvider::AttributeName);
}

void UAnimGraphNode_Steering::GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	OutAttributes.Add(UE::Anim::IAnimRootMotionProvider::AttributeName);
}

void UAnimGraphNode_Steering::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);
}

#undef LOCTEXT_NAMESPACE
