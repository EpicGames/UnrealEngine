// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraph/AnimGraphNode_StrideWarping.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_StrideWarping)

#define LOCTEXT_NAMESPACE "AnimationWarping"

UAnimGraphNode_StrideWarping::UAnimGraphNode_StrideWarping(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

FText UAnimGraphNode_StrideWarping::GetControllerDescription() const
{
	return LOCTEXT("StrideWarping", "Stride Warping");
}

FText UAnimGraphNode_StrideWarping::GetTooltipText() const
{
	return LOCTEXT("StrideWarpingTooltip", "Adjusts the speed of the animation by warping the leg stride with the specified scale and direction.");
}

FText UAnimGraphNode_StrideWarping::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return GetControllerDescription();
}

FLinearColor UAnimGraphNode_StrideWarping::GetNodeTitleColor() const
{
	return FLinearColor(FColor(153.f, 0.f, 0.f));
}

void UAnimGraphNode_StrideWarping::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	Super::CustomizePinData(Pin, SourcePropertyName, ArrayIndex);

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_StrideWarping, StrideDirection))
	{
		Pin->bHidden = (Node.Mode == EWarpingEvaluationMode::Graph);
	}

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_StrideWarping, StrideScale))
	{
		Pin->bHidden = (Node.Mode == EWarpingEvaluationMode::Graph);
	}

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_StrideWarping, LocomotionSpeed))
	{
		Pin->bHidden = (Node.Mode == EWarpingEvaluationMode::Manual);
	}

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_StrideWarping, MinRootMotionSpeedThreshold))
	{
		Pin->bHidden = (Node.Mode == EWarpingEvaluationMode::Manual);
	}
}

void UAnimGraphNode_StrideWarping::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	Super::CustomizeDetails(DetailBuilder);

	DetailBuilder.SortCategories([](const TMap<FName, IDetailCategoryBuilder*>& CategoryMap)
	{
		for (const TPair<FName, IDetailCategoryBuilder*>& Pair : CategoryMap)
		{
			int32 SortOrder = Pair.Value->GetSortOrder();
			const FName CategoryName = Pair.Key;

			if (CategoryName == "Evaluation")
			{
				SortOrder += 1;
			}
			else if (CategoryName == "Settings")
			{
				SortOrder += 2;
			}
			else if (CategoryName == "Advanced")
			{
				SortOrder += 3;
			}
			else if (CategoryName == "Debug")
			{
				SortOrder += 4;
			}
			else
			{
				const int32 ValueSortOrder = Pair.Value->GetSortOrder();
				if (ValueSortOrder >= SortOrder && ValueSortOrder < SortOrder + 10)
				{
					SortOrder += 10;
				}
				else
				{
					continue;
				}
			}

			Pair.Value->SetSortOrder(SortOrder);
		}
	});

	TSharedRef<IPropertyHandle> NodeHandle = DetailBuilder.GetProperty(FName(TEXT("Node")), GetClass());

	if (Node.Mode == EWarpingEvaluationMode::Graph)
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_StrideWarping, StrideScale)));
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_StrideWarping, StrideDirection)));
	}

	if (Node.Mode == EWarpingEvaluationMode::Manual)
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_StrideWarping, LocomotionSpeed)));
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_StrideWarping, MinRootMotionSpeedThreshold)));
	}
}

void UAnimGraphNode_StrideWarping::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	Super::PostEditChangeProperty(PropertyChangedEvent);

	bool bRequiresNodeReconstruct = false;
	FProperty* ChangedProperty = PropertyChangedEvent.Property;

	if (ChangedProperty)
	{
		if ((ChangedProperty->GetFName() == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bMapRange))
			|| (ChangedProperty->GetFName() == GET_MEMBER_NAME_STRING_CHECKED(FInputRange, Min))
			|| (ChangedProperty->GetFName() == GET_MEMBER_NAME_STRING_CHECKED(FInputRange, Max))
			|| (ChangedProperty->GetFName() == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, Scale))
			|| (ChangedProperty->GetFName() == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, Bias))
			|| (ChangedProperty->GetFName() == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bClampResult))
			|| (ChangedProperty->GetFName() == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, ClampMin))
			|| (ChangedProperty->GetFName() == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, ClampMax))
			|| (ChangedProperty->GetFName() == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, bInterpResult))
			|| (ChangedProperty->GetFName() == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, InterpSpeedIncreasing))
			|| (ChangedProperty->GetFName() == GET_MEMBER_NAME_STRING_CHECKED(FInputScaleBiasClamp, InterpSpeedDecreasing)))
		{
			bRequiresNodeReconstruct = true;
		}

		// Evaluation mode
		if (ChangedProperty->GetFName() == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_StrideWarping, Mode))
		{
			FScopedTransaction Transaction(LOCTEXT("ChangeEvaluationMode", "Change Evaluation Mode"));
			Modify();

			// Break links to pins going away
			for (int32 PinIndex = 0; PinIndex < Pins.Num(); ++PinIndex)
			{
				UEdGraphPin* Pin = Pins[PinIndex];
				if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_StrideWarping, StrideDirection))
				{
					if (Node.Mode == EWarpingEvaluationMode::Graph)
					{
						Pin->BreakAllPinLinks();
					}
				}
				else if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_StrideWarping, StrideScale))
				{
					if (Node.Mode == EWarpingEvaluationMode::Graph)
					{
						Pin->BreakAllPinLinks();
					}
				}
				else if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_StrideWarping, LocomotionSpeed))
				{
					if (Node.Mode == EWarpingEvaluationMode::Manual)
					{
						Pin->BreakAllPinLinks();
					}
				}
				else if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_StrideWarping, MinRootMotionSpeedThreshold))
				{
					if (Node.Mode == EWarpingEvaluationMode::Manual)
					{
						Pin->BreakAllPinLinks();
					}
				}
			}

			bRequiresNodeReconstruct = true;
		}
	}

	if (bRequiresNodeReconstruct)
	{
		ReconstructNode();
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}
}

void UAnimGraphNode_StrideWarping::GetInputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	if (Node.Mode == EWarpingEvaluationMode::Graph)
	{
		OutAttributes.Add(UE::Anim::IAnimRootMotionProvider::AttributeName);
	}
}

void UAnimGraphNode_StrideWarping::GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	if (Node.Mode == EWarpingEvaluationMode::Graph)
	{
		OutAttributes.Add(UE::Anim::IAnimRootMotionProvider::AttributeName);
	}
}

void UAnimGraphNode_StrideWarping::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	auto HasInvalidBoneName = [](const FName& BoneName) 
	{ 
		return BoneName == NAME_None; 
	};

	auto HasInvalidBoneIndex = [&] (const FName& BoneName) 
	{ 
		return ForSkeleton && ForSkeleton->GetReferenceSkeleton().FindBoneIndex(BoneName) == INDEX_NONE; 
	};

	auto InvalidBoneNameMessage = [&](const FName& BoneName) 
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("BoneName"), FText::FromName(BoneName));
		const FText Message = FText::Format(NSLOCTEXT("StrideWarping", "Invalid{BoneName}BoneName", "@@ - {BoneName} bone not found in Skeleton"), Args);
		MessageLog.Warning(*Message.ToString(), this);
	};

	auto InvalidBoneIndexMessage = [&](const FName& BoneName) 
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("BoneName"), FText::FromName(BoneName));
		const FText Message = FText::Format(NSLOCTEXT("StrideWarping", "Invalid{BoneName}BoneInSkeleton", "@@ - {BoneName} bone definition is required"), Args);
		MessageLog.Warning(*Message.ToString(), this);
	};

	if (HasInvalidBoneName(Node.PelvisBone.BoneName))
	{
		InvalidBoneIndexMessage("Pelvis");
	}
	else if (HasInvalidBoneIndex(Node.PelvisBone.BoneName))
	{
		InvalidBoneNameMessage(Node.PelvisBone.BoneName);
	}

	if (HasInvalidBoneName(Node.IKFootRootBone.BoneName))
	{
		InvalidBoneIndexMessage("IK Foot Root");
	}
	else if (HasInvalidBoneIndex(Node.IKFootRootBone.BoneName))
	{
		InvalidBoneNameMessage(Node.IKFootRootBone.BoneName);
	}

	if (Node.FootDefinitions.IsEmpty())
	{
		MessageLog.Warning(*NSLOCTEXT("StrideWarping", "InvalidFootDefinitions", "@@ - Foot definitions are required").ToString(), this);
	}
	else
	{
		for (const auto& Foot : Node.FootDefinitions)
		{
			if (HasInvalidBoneName(Foot.IKFootBone.BoneName))
			{
				InvalidBoneIndexMessage("IK Foot");
			}
			else if (HasInvalidBoneIndex(Foot.IKFootBone.BoneName))
			{
				InvalidBoneNameMessage(Foot.IKFootBone.BoneName);
			}

			if (HasInvalidBoneName(Foot.FKFootBone.BoneName))
			{
				InvalidBoneIndexMessage("FK Foot");
			}
			else if (HasInvalidBoneIndex(Foot.FKFootBone.BoneName))
			{
				InvalidBoneNameMessage(Foot.FKFootBone.BoneName);
			}

			if (HasInvalidBoneName(Foot.ThighBone.BoneName))
			{
				InvalidBoneIndexMessage("Thigh");
			}
			else if (HasInvalidBoneIndex(Foot.ThighBone.BoneName))
			{
				InvalidBoneNameMessage(Foot.ThighBone.BoneName);
			}
		}
	}

	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);
}

#undef LOCTEXT_NAMESPACE
