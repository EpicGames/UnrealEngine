// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraph/AnimGraphNode_WarpTest.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_WarpTest)

#define LOCTEXT_NAMESPACE "AnimationWarping"

UAnimGraphNode_WarpTest::UAnimGraphNode_WarpTest(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

FText UAnimGraphNode_WarpTest::GetTooltipText() const
{
	return LOCTEXT("WarpTestTooltip", "Warp Test Node");
}

FText UAnimGraphNode_WarpTest::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("WarpTest", "Warp Test Node");
}

FLinearColor UAnimGraphNode_WarpTest::GetNodeTitleColor() const
{
	return FLinearColor(FColor(153.f, 0.f, 0.f));
}

#undef LOCTEXT_NAMESPACE
