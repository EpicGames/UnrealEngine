// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationModifiers/OrientationWarpingModifier.h"
#include "Animation/AnimSequence.h"
#include "AnimationBlueprintLibrary.h"
#include "EngineLogs.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OrientationWarpingModifier)

#define LOCTEXT_NAMESPACE "OrientationWarpingModifier"

void UOrientationWarpingModifier::OnApply_Implementation(UAnimSequence* Animation)
{
	if (Animation == nullptr)
	{
		UE_LOGF(LogAnimation, Error, "OrientationWarpingModifier failed. Reason: Invalid Animation");
		return;
	}

	USkeleton* Skeleton = Animation->GetSkeleton();
	if (Skeleton == nullptr)
	{
		UE_LOGF(LogAnimation, Error, "OrientationWarpingModifier failed. Reason: Animation with invalid Skeleton. Animation: %ls",
			*GetNameSafe(Animation));
		return;
	}
	
	if ((EnableOffsetCurveName == NAME_None) || (EnableWarpingCurveName == NAME_None))
	{
		UE_LOGF(LogAnimation, Error, "OrientationWarpingModifier failed. Reason: Name==None on curve. Animation: %ls",
			*GetNameSafe(Animation));
		return;
	}

	bool bRemoveNameFromSkeleton = false;

	const bool bMetaDataCurve = false;
	UAnimationBlueprintLibrary::AddCurve(Animation, EnableOffsetCurveName, ERawCurveTrackTypes::RCT_Float, bMetaDataCurve);
	UAnimationBlueprintLibrary::AddCurve(Animation, EnableWarpingCurveName, ERawCurveTrackTypes::RCT_Float, bMetaDataCurve);

	const float AnimLength = Animation->GetPlayLength();
	float SampleInterval = 1.f / 120.f;
	int32 NumSteps = AnimLength / SampleInterval;

	bool bIsRotating = false;
	bool bFoundStop = false;
	TArray<float> Keys;
	Keys.Reserve(10);
	TArray<float> Values;
	Values.Reserve(10);
	for (int32 Step = 0; Step < NumSteps; ++Step)
	{
		const float Time = Step * SampleInterval;

		constexpr bool bAllowLooping = false;
		const FAnimExtractContext Context(static_cast<double>(Time), true, FDeltaTimeRecord(SampleInterval), bAllowLooping);
		const FTransform RootMotion = Animation->ExtractRootMotion(Context);
		const FVector RootMotionTranslation = RootMotion.GetTranslation();
		const FQuat RootMotionRotation = RootMotion.GetRotation();

		FVector RotationAxis;
		float RotationAngle;
		RootMotionRotation.ToAxisAndAngle(RotationAxis, RotationAngle);
		const float RootMotionSpeed = RootMotionTranslation.Size2D() / SampleInterval;

		const bool bNewIsMoving = RootMotionSpeed > StopSpeedThreshold;
		const bool bNewIsRotating = FMath::Abs(RotationAngle) > KINDA_SMALL_NUMBER;

		// Only add keys at the beginning/end of rotations
		//@TODO: Check for additional info to decide when to advance the curve i.e. minimize corrections when feet are planted.
		if (bNewIsRotating != bIsRotating)
		{
			// Blend in should be outside the rotation window to prevent missing out on significant rotation.
			// Blend out should occur during rotation.
			// @TODO: From tests, it doesn't look bad to extend blend out past the rotation window. Investigate this
			const float NewValue = bNewIsRotating ? 1.0f : 0.0f;
			const float TimeWithBlend = FMath::Clamp(Time - (bNewIsRotating ? BlendInTime : BlendOutTime), 0.0f, AnimLength);
			const float BlendKeyValue = (!bNewIsMoving && bNewIsRotating) ? 1.0f : 1.0f - NewValue;
			Keys.Add(TimeWithBlend);
			Values.Add(BlendKeyValue);
			Keys.Add(Time);
			Values.Add(NewValue);
		}
		bIsRotating = bNewIsRotating;

		bFoundStop |= !bNewIsMoving;
	}

	if (bFoundStop && !Keys.IsEmpty())
	{
		UAnimationBlueprintLibrary::AddFloatCurveKeys(Animation, EnableOffsetCurveName, Keys, Values);


		for (int32 KeyIndex = 1; KeyIndex < Keys.Num(); ++KeyIndex)
		{
			if ((Values[KeyIndex] == 0.0f) && (Values[KeyIndex] != Values[KeyIndex - 1]))
			{
				// Extend blends to start after rotation ends
				const float Prev = Keys[KeyIndex - 1];
				const float Next = Keys[KeyIndex];

				const float BlendTime = Next-Prev;
				Keys[KeyIndex - 1] = Next;
				Keys[KeyIndex] = Next + BlendTime;
			}
		}
		for (int32 KeyIndex = 0; KeyIndex < Keys.Num(); ++KeyIndex)
		{
			// Invert values for warping alpha 
			// @TODO: It doesn't look bad to enable pose warping near the end of the blend sometimes.
			// Result is that the delta is corrected by strafe, instead of forward-facing.
			// Figure out how to detect when this is valid automatically.
			Values[KeyIndex] = 1.0f - Values[KeyIndex];
		}

		UAnimationBlueprintLibrary::AddFloatCurveKeys(Animation, EnableWarpingCurveName, Keys, Values);
	}
	else
	{
		UAnimationBlueprintLibrary::AddFloatCurveKey(Animation, EnableOffsetCurveName, 0.0f, 0.0f);
		UAnimationBlueprintLibrary::AddFloatCurveKey(Animation, EnableWarpingCurveName, 0.0f, 1.0f);
	}
}

void UOrientationWarpingModifier::OnRevert_Implementation(UAnimSequence* Animation)
{
	const bool bRemoveNameFromSkeleton = false;
	UAnimationBlueprintLibrary::RemoveCurve(Animation, EnableWarpingCurveName, bRemoveNameFromSkeleton);
	UAnimationBlueprintLibrary::RemoveCurve(Animation, EnableOffsetCurveName, bRemoveNameFromSkeleton);
}

#undef LOCTEXT_NAMESPACE
