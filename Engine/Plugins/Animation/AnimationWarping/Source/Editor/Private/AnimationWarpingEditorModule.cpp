// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationWarpingEditorModule.h"
#include "Modules/ModuleManager.h"

class FAnimationWarpingEditorModule : public IAnimationWarpingEditorModule
{
};


IMPLEMENT_MODULE( FAnimationWarpingEditorModule, AnimationWarpingEditor )
