// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphNode_Base.h"
#include "BoneControllers/AnimNode_OffsetRootBone.h"

#include "AnimGraphNode_OffsetRootBone.generated.h"

#define UE_API ANIMATIONWARPINGEDITOR_API

namespace ENodeTitleType { enum Type : int; }

UCLASS(MinimalAPI, Experimental)
class UAnimGraphNode_OffsetRootBone : public UAnimGraphNode_Base
{
	GENERATED_UCLASS_BODY()


	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_OffsetRootBone Node;

public:
	// UEdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void GetInputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
	UE_API virtual void GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
	// End of UEdGraphNode interface

protected:
	// UAnimGraphNode_Base interface
	UE_API virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const override;
	// End of UAnimGraphNode_Base interface

};

#undef UE_API
