// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphNode_Base.h"
#include "BoneControllers/AnimNode_OverrideRootMotion.h"

#include "AnimGraphNode_OverrideRootMotion.generated.h"

#define UE_API ANIMATIONWARPINGEDITOR_API

namespace ENodeTitleType { enum Type : int; }

UCLASS(MinimalAPI, Experimental)
class UAnimGraphNode_OverrideRootMotion : public UAnimGraphNode_Base
{
	GENERATED_UCLASS_BODY()


	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_OverrideRootMotion Node;

public:
	// UEdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual void GetInputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
	UE_API virtual void GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
	// End of UEdGraphNode interface

};

#undef UE_API
