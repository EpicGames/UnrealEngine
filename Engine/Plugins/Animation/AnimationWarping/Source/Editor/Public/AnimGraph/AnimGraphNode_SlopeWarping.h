// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphNode_SkeletalControlBase.h"
#include "BoneControllers/AnimNode_SlopeWarping.h"

#include "AnimGraphNode_SlopeWarping.generated.h"

#define UE_API ANIMATIONWARPINGEDITOR_API

namespace ENodeTitleType { enum Type : int; }

UCLASS(MinimalAPI, Experimental)
class UAnimGraphNode_SlopeWarping : public UAnimGraphNode_SkeletalControlBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_SlopeWarping Node;

public:
	// UEdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	// End of UEdGraphNode interface

protected:
	// UAnimGraphNode_Base interface
	// End of UAnimGraphNode_Base interface

	// UAnimGraphNode_SkeletalControlBase interface
	UE_API virtual FText GetControllerDescription() const override;
	virtual const FAnimNode_SkeletalControlBase* GetNode() const override { return &Node; }
	// End of UAnimGraphNode_SkeletalControlBase interface
};

#undef UE_API
