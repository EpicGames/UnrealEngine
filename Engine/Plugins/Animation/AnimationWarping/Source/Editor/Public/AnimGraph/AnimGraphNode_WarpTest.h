// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphNode_Base.h"
#include "BoneControllers/AnimNode_WarpTest.h"
#include "AnimGraphNode_WarpTest.generated.h"

#define UE_API ANIMATIONWARPINGEDITOR_API

namespace ENodeTitleType { enum Type : int; }

UCLASS(MinimalAPI, Experimental)
class UAnimGraphNode_WarpTest : public UAnimGraphNode_Base
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_WarpTest Node;

public:
	// UEdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetTooltipText() const override;
	// End of UEdGraphNode interface
};

#undef UE_API
