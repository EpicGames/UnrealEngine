// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationModifier.h"
#include "OrientationWarpingModifier.generated.h"

class UAnimSequence;

//@TODO: Add comments/tooltips.
UCLASS(Experimental)
class UOrientationWarpingModifier : public UAnimationModifier
{
	GENERATED_BODY()

public:

	virtual void OnApply_Implementation(UAnimSequence* Animation) override;
	virtual void OnRevert_Implementation(UAnimSequence* Animation) override;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Experimental)
	FName EnableWarpingCurveName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Experimental)
	FName EnableOffsetCurveName;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Experimental)
	float BlendInTime = 0.25f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Experimental)
	float BlendOutTime = 0.25f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Experimental)
	float StopSpeedThreshold = 10.0f;
};
