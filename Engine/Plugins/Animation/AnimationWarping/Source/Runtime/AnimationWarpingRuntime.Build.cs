// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AnimationWarpingRuntime : ModuleRules
	{
		public AnimationWarpingRuntime(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AnimationCore",
                "AnimGraphRuntime",
                "Core",
                "CoreUObject",
                "Engine",
            });
		}
	}
}
