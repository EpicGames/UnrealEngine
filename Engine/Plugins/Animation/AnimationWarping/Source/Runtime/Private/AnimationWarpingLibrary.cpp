// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationWarpingLibrary.h"
#include "BoneControllers/AnimNode_OffsetRootBone.h"
#include "Animation/AnimMontage.h"
#include "Curves/CurveFloat.h"
#include "Curves/CurveVector.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationWarpingLibrary)

FTransform UAnimationWarpingLibrary::GetOffsetRootTransform(const FAnimNodeReference& Node)
{
	FTransform Transform(FTransform::Identity);

	if (FAnimNode_OffsetRootBone* OffsetRoot = Node.GetAnimNodePtr<FAnimNode_OffsetRootBone>())
	{
		OffsetRoot->GetOffsetRootTransform(Transform);
	}

	return Transform;
}

bool UAnimationWarpingLibrary::GetCurveValueFromAnimation(const UAnimSequenceBase* Animation, FName CurveName, float Time, float& OutValue)
{
	OutValue = 0.0f;

	// If Animation is a Montage we need to get the AnimSequence at the desired time
	// because EvaluateCurveData doesn't work when called from a Montage and the curve is in the AnimSequence within the montage. 
	if (const UAnimMontage* Montage = Cast<UAnimMontage>(Animation))
	{
		// For now just assume we are working with a montage with a single slot, which is the most common case anyway
		// The engine also makes this assumption some times. E.g Root motion is only extracted from the first track See: UAnimMontage::ExtractRootMotionFromTrackRange
		if (Montage->SlotAnimTracks.Num() > 0)
		{
			const FAnimTrack& AnimTrack = Montage->SlotAnimTracks[0].AnimTrack;

			if (const FAnimSegment* Segment = AnimTrack.GetSegmentAtTime(Time))
			{
				if (Segment->GetAnimReference() && Segment->GetAnimReference()->HasCurveData(CurveName))
				{
					float ActualTime = Segment->ConvertTrackPosToAnimPos(Time);
					ActualTime = FMath::Clamp(ActualTime, Segment->AnimStartTime, Segment->AnimEndTime);

					const FAnimExtractContext Context(static_cast<double>(ActualTime));
					OutValue = Segment->GetAnimReference()->EvaluateCurveData(CurveName, Context);
					return true;
				}
			}
		}
	}
	else if (Animation && Animation->HasCurveData(CurveName))
	{
		const FAnimExtractContext Context(static_cast<double>(Time));
		OutValue = Animation->EvaluateCurveData(CurveName, Context);
		return true;
	}

	return false;
}

bool UAnimationWarpingLibrary::GetFloatValueFromCurve(const UCurveFloat* InCurve, float InTime, float& OutValue)
{
	if (InCurve != nullptr)
	{
		OutValue = InCurve->GetFloatValue(InTime);
		return true;
	}
	
	return false;
}

bool UAnimationWarpingLibrary::GetVectorValueFromCurve(const UCurveVector* InCurve, float InTime, FVector& OutValue)
{
	if (InCurve != nullptr)
	{
		OutValue = InCurve->GetVectorValue(InTime);
		return true;
	}
	
	return false;
}

void UAnimationWarpingLibrary::ConvertToOffsetRootBoneNodePure(const FAnimNodeReference& Node, FOffsetRootBoneAnimNodeReference& OffsetRootBoneNode,
	bool& Result)
{
	EAnimNodeReferenceConversionResult ConversionResult;
	OffsetRootBoneNode = FAnimNodeReference::ConvertToType<FOffsetRootBoneAnimNodeReference>(Node, ConversionResult);
	Result = (ConversionResult == EAnimNodeReferenceConversionResult::Succeeded);
}

void UAnimationWarpingLibrary::ApplyDeltaToOffsetRootBone(const FOffsetRootBoneAnimNodeReference& OffsetRootBoneNode, FVector TranslationWorld,
                                                          FQuat RotationWorld)
{
	if (FAnimNode_OffsetRootBone* Node = OffsetRootBoneNode.GetAnimNodePtr<FAnimNode_OffsetRootBone>())
	{
		FVector SimulatedTranslation = Node->GetOffsetRootTranslation();
		FQuat SimulatedRotation = Node->GetOffsetRootRotation();

		SimulatedTranslation += TranslationWorld;
		SimulatedRotation = RotationWorld * SimulatedRotation;

		Node->SetOffsetRootTranslation(SimulatedTranslation);
		Node->SetOffsetRootRotation(SimulatedRotation);
	}
}
