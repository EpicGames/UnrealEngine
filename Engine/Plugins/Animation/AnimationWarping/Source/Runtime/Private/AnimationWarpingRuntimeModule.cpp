// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationWarpingRuntimeModule.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Animation/AnimSequence.h"
#include "Animation/AttributesRuntime.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"

class FAnimationWarpingRuntimeModule : public IAnimationWarpingRuntimeModule
{
};

namespace UE { namespace AnimationWarping {

class FModule : public IModuleInterface, public UE::Anim::IAnimRootMotionProvider
{
public:

	FModule();
	
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

public:
	virtual void SampleRootMotion(const FDeltaTimeRecord& SampleRange, const UAnimSequence& Sequence, bool bLoopingSequence, UE::Anim::FStackAttributeContainer& OutAttributes) const override;
	virtual bool SetRootMotion(const FTransform& RootMotionDelta, UE::Anim::FStackAttributeContainer& OutAttributes) const override;
	virtual bool OverrideRootMotion(const FTransform& RootMotionDelta, UE::Anim::FStackAttributeContainer& OutAttributes) const override;
	virtual bool ExtractRootMotion(const UE::Anim::FStackAttributeContainer& Attributes, FTransform& OutRootMotionDelta) const override;
	virtual bool HasRootMotion(const UE::Anim::FStackAttributeContainer& Attributes) const override;

	// Root motion animation attribute data will always be accessible on the root bone's attribute container
	const UE::Anim::FAttributeId RootMotionAttributeId;
};

FModule::FModule()
	: RootMotionAttributeId(UE::Anim::IAnimRootMotionProvider::AttributeName , FCompactPoseBoneIndex(0))
{
}

void FModule::StartupModule()
{
	IModularFeatures::Get().RegisterModularFeature(UE::Anim::IAnimRootMotionProvider::ModularFeatureName, this);
}

void FModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(UE::Anim::IAnimRootMotionProvider::ModularFeatureName, this);
}

void FModule::SampleRootMotion(const FDeltaTimeRecord& SampleRange, const UAnimSequence& Sequence, bool bLoopingSequence, UE::Anim::FStackAttributeContainer& OutAttributes) const
{
	const FAnimExtractContext Context(static_cast<double>(SampleRange.GetPrevious()), true, FDeltaTimeRecord(SampleRange.Delta), bLoopingSequence);
	const FTransform RootMotionTransform = Sequence.ExtractRootMotion(Context);
	FTransformAnimationAttribute* RootMotionAttribute = OutAttributes.FindOrAdd<FTransformAnimationAttribute>(RootMotionAttributeId);
	RootMotionAttribute->Value = RootMotionTransform;
}

bool FModule::SetRootMotion(const FTransform& RootMotionDelta, UE::Anim::FStackAttributeContainer& OutAttributes) const
{
	check(RootMotionDelta.IsRotationNormalized());
	if (FTransformAnimationAttribute* RootMotionAttribute = OutAttributes.FindOrAdd<FTransformAnimationAttribute>(RootMotionAttributeId))
	{
		RootMotionAttribute->Value = RootMotionDelta;
		return true;
	}

	return false;
}

bool FModule::OverrideRootMotion(const FTransform& RootMotionDelta, UE::Anim::FStackAttributeContainer& OutAttributes) const
{
	check(RootMotionDelta.IsRotationNormalized());
	if (FTransformAnimationAttribute* RootMotionAttribute = OutAttributes.Find<FTransformAnimationAttribute>(RootMotionAttributeId))
	{
		RootMotionAttribute->Value = RootMotionDelta;
		return true;
	}

	return false;
}

bool FModule::ExtractRootMotion(const UE::Anim::FStackAttributeContainer& Attributes, FTransform& OutRootMotionDelta) const
{
	const FTransformAnimationAttribute* RootMotionAttribute = Attributes.Find<FTransformAnimationAttribute>(RootMotionAttributeId);
	OutRootMotionDelta = RootMotionAttribute ? RootMotionAttribute->Value : FTransform::Identity;
	return !!RootMotionAttribute;
}

bool FModule::HasRootMotion(const UE::Anim::FStackAttributeContainer& Attributes) const
{
	return !!Attributes.Find<FTransformAnimationAttribute>(RootMotionAttributeId);
}

}}

IMPLEMENT_MODULE(UE::AnimationWarping::FModule, AnimationWarpingRuntime)
