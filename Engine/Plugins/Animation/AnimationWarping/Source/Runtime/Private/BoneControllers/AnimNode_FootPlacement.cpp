// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_FootPlacement.h"
#include "Animation/AnimNodeFunctionRef.h"
#include "Animation/AnimTrace.h"
#include "AnimationRuntime.h"
#include "Animation/AnimInstanceProxy.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/World.h"
#include "GameFramework/Character.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "Animation/AnimRootMotionProvider.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_FootPlacement)

DECLARE_CYCLE_STAT(TEXT("Foot Placement Eval"), STAT_FootPlacement_Eval, STATGROUP_Anim);

#if ENABLE_FOOTPLACEMENT_DEBUG
static TAutoConsoleVariable<bool> CVarAnimNodeFootPlacementEnable(TEXT("a.AnimNode.FootPlacement.Enable"), true, TEXT("Enable/Disable Foot Placement"));
static TAutoConsoleVariable<bool> CVarAnimNodeFootPlacementEnableLock(TEXT("a.AnimNode.FootPlacement.Enable.Lock"), true, TEXT("Enable/Disable Foot Locking"));
static TAutoConsoleVariable<bool> CVarAnimNodeFootPlacementDebug(TEXT("a.AnimNode.FootPlacement.Debug"), false, TEXT("Turn on visualization debugging for Foot Placement"));
static TAutoConsoleVariable<bool> CVarAnimNodeFootPlacementDebugTraces(TEXT("a.AnimNode.FootPlacement.Debug.Traces"), false, TEXT("Turn on visualization debugging for foot ground traces"));
static TAutoConsoleVariable<int> CVarAnimNodeFootPlacementDebugDrawHistory(TEXT("a.AnimNode.FootPlacement.Debug.DrawHistory"), 0,
	TEXT("Turn on history visualization debugging 0 = Disabled, -1 = Pelvis, >1 = Foot Index. Clear with FlushPersistentDebugLines"));
#endif

namespace UE::Anim::FootPlacement
{
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	/// UE::Anim::FootPlacement::FEvaluationContext
	struct FEvaluationContext
	{
		FEvaluationContext(
			FComponentSpacePoseContext& InCSPContext,
			const FVector& InApproachDirWS,
			const float InUpdateDeltaTime);

		FComponentSpacePoseContext& CSPContext;

		class AActor* OwningActor = nullptr;
		class UWorld* World = nullptr;
		//@TODO: Make everything character related be input pins, to make this character mover agnostic. Not of the accesses are thread-safe technically.
		class UCharacterMovementComponent* MovementComponent = nullptr;
		FTransform OwningComponentToWorld = FTransform::Identity;
		FTransform RootMotionTransformDelta = FTransform::Identity;
		float UpdateDeltaTime = 0.0f;
		FVector ApproachDirWS = -FVector::UpVector;
		FVector ApproachDirCS = -FVector::UpVector;

		FVector GetMovementComponentFloorNormal() const
		{
			if (!MovementComponent)
			{
				return -ApproachDirWS;
			}

			return (MovementComponent->CurrentFloor.bBlockingHit) ?
				MovementComponent->CurrentFloor.HitResult.ImpactNormal : -ApproachDirWS;
		}

		FVector GetMovementComponentFloorLocation() const
		{
			if (!MovementComponent)
			{
				return OwningComponentToWorld.GetLocation();
			}

			return (MovementComponent->CurrentFloor.bBlockingHit) ?
				MovementComponent->CurrentFloor.HitResult.ImpactPoint : OwningComponentToWorld.GetLocation();
		}

		bool GetMovementComponentIsWalkable(const FHitResult& InHit) const
		{
			if (!MovementComponent)
			{
				return false;
			}

			return MovementComponent->IsWalkable(InHit);
		}

		FVector GetCharacterVelocity() const
		{
			return OwningActor->GetVelocity();
		}
	};

	FEvaluationContext::FEvaluationContext(
		FComponentSpacePoseContext& InCSPContext,
		const FVector& InApproachDirCS,
		const float InUpdateDeltaTime)
		: CSPContext(InCSPContext)
		, UpdateDeltaTime(InUpdateDeltaTime)
		, ApproachDirCS(InApproachDirCS)
	{
		const USkeletalMeshComponent* OwningComponent = CSPContext.AnimInstanceProxy->GetSkelMeshComponent();
		OwningActor = OwningComponent->GetOwner();
		World = OwningComponent->GetWorld();

		ACharacter* CharacterOwner = Cast<ACharacter>(OwningActor);
		MovementComponent = CharacterOwner ? CharacterOwner->GetCharacterMovement() : nullptr;
		OwningComponentToWorld = OwningComponent->GetComponentToWorld();

		ApproachDirWS = OwningComponentToWorld.TransformVectorNoScale(ApproachDirCS);
	
		RootMotionTransformDelta = FTransform::Identity;
		if (const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get())
		{
			RootMotionProvider->ExtractRootMotion(CSPContext.CustomAttributes, RootMotionTransformDelta);
		}
	}

	static FVector ReOrientNormal(const FVector& ApproachDir, const FVector& InNormal, FVector& PointA, const FVector& PointB)
	{
		const FVector AxisX = (PointA - PointB).GetSafeNormal();
		if (!AxisX.IsNearlyZero() && !InNormal.IsNearlyZero() && (FMath::Abs(AxisX | InNormal) > DELTA))
		{
			const FVector AxisY = AxisX ^ InNormal;
			const FVector AxisZ = AxisX ^ AxisY;

			// Make sure our normal points upwards. (take into account gravity dir?)
			return ((AxisZ | -ApproachDir) > 0.f) ? AxisZ : -AxisZ;
		}

		return InNormal;
	}

	// Since we are calculating in world-space, when too far from the origin FMath::LinePlaneIntersection can introduce numerical error
	// and consider the line's start/end to be at the same location.
	// Use point-direction instead to avoid this
	// See UE-162275
	static FVector PointDirectionPlaneIntersection(const FVector Point, const FVector Direction, const FPlane Plane)
	{
		return
			Point
			+ Direction
			* ((Plane.W - (Point | Plane)) / (Direction | Plane));
	};

	static bool FindPlantTraceImpact(
		const FEvaluationContext& Context,
		const FFootPlacementTraceSettings& TraceSettings,
		const bool bCheckComplex,
		const FVector& StartPositionWS,
		FVector& OutImpactLocationWS,
		FVector& OutImpactNormalWS)
	{
		OutImpactLocationWS = Context.OwningComponentToWorld.GetLocation();
		OutImpactNormalWS = Context.OwningComponentToWorld.GetRotation().GetUpVector();

		if (!IsValid(Context.World) || !TraceSettings.bEnabled)
		{
			return false;
		}

		const FCollisionShape CollisionShape = FCollisionShape::MakeSphere(TraceSettings.SweepRadius);

		const FVector TraceDirectionWS = Context.ApproachDirWS;
		const FVector TraceStart = StartPositionWS + (TraceDirectionWS * TraceSettings.StartOffset);
		const FVector TraceEnd = StartPositionWS + (TraceSettings.EndOffset * TraceDirectionWS);

		FCollisionQueryParams QueryParams;
		QueryParams.bTraceComplex = bCheckComplex;
		// Ignore self and all attached components
		QueryParams.AddIgnoredActor(Context.OwningActor);

		const ECollisionChannel CollisionChannel = UEngineTypes::ConvertToCollisionChannel(bCheckComplex ? TraceSettings.ComplexTraceChannel : TraceSettings.SimpleTraceChannel);

		FHitResult HitResult;
		const bool bHit = Context.World->SweepSingleByChannel(
			HitResult, TraceStart, TraceEnd, FQuat::Identity, CollisionChannel, CollisionShape, QueryParams);

		if (!bHit || !Context.GetMovementComponentIsWalkable(HitResult))
		{
			// If the hit fails or isn't walkable, use the ground plane position and a default impact normal (negated trace direction)
			OutImpactLocationWS = PointDirectionPlaneIntersection(StartPositionWS, TraceDirectionWS, FPlane(Context.GetMovementComponentFloorLocation(), -TraceDirectionWS));
			OutImpactNormalWS = -TraceDirectionWS;
			return false;
		}

		OutImpactLocationWS = HitResult.ImpactPoint;
		OutImpactNormalWS = HitResult.ImpactNormal;

#if ENABLE_FOOTPLACEMENT_DEBUG
		if (CVarAnimNodeFootPlacementDebugTraces.GetValueOnAnyThread())
		{
			Context.CSPContext.AnimInstanceProxy->AnimDrawDebugPoint(
				TraceStart, 10.0f, FColor::Purple, false, -1.0f, SDPG_Foreground);

			Context.CSPContext.AnimInstanceProxy->AnimDrawDebugPoint(
				OutImpactLocationWS, 10.0f, FColor::Purple, false, -1.0f, SDPG_Foreground);

			Context.CSPContext.AnimInstanceProxy->AnimDrawDebugLine(
				TraceStart, OutImpactLocationWS,
				FColor::Purple,
				false, -1.0f, 1.0f, SDPG_Foreground);
		}
#endif

		return true;
	}


	static bool FindPlantPlane(const FEvaluationContext& Context,
		const FFootPlacementTraceSettings& TraceSettings,
		const FVector& StartPositionWS,
		const bool bCheckComplex,
		FPlane& OutPlantPlaneWS,
		FVector& ImpactLocationWS)
	{
		FVector ImpactNormal;
		const bool bFound = FindPlantTraceImpact(Context, TraceSettings, bCheckComplex, StartPositionWS, ImpactLocationWS, ImpactNormal);
		OutPlantPlaneWS = FPlane(ImpactLocationWS, ImpactNormal);

		return bFound;
	}

	static FVector CalculateCentroid(const TArrayView<FTransform>& Transforms)
	{
		check(Transforms.Num() > 0);

		FVector Centroid = FVector::ZeroVector;
		for (const FTransform& Transform : Transforms)
		{
			Centroid += Transform.GetLocation();
		}

		Centroid /= float(Transforms.Num());
		return Centroid;
	}

	static TOptional<float> GetDistanceToPlaneAlongDirection(const FVector& Location, const FPlane& PlantPlane, const FVector& ApproachDir)
	{
		// If approach dir is perpendicular to plant plane, distance is undefined
		if (FMath::IsNearlyZero(ApproachDir | PlantPlane))
		{
			return TOptional<float>();
		}
		
		const FVector IntersectionLoc = UE::Anim::FootPlacement::PointDirectionPlaneIntersection(
			Location,
			-ApproachDir,
			PlantPlane);

		const FVector IntersectionToLocation = Location - IntersectionLoc;
		const float DistanceToPlantPlane = IntersectionToLocation | -ApproachDir;
		return DistanceToPlantPlane;
	}

	static void FindChainLengthRootBoneIndex(
		const FCompactPoseBoneIndex& InFootBoneIndex,
		const int32& NumBonesInLimb,
		const FBoneContainer& RequiredBones,
		FCompactPoseBoneIndex& OutHipIndex,
		float& OutChainLength)
	{
		OutChainLength = 0.0f;
		FCompactPoseBoneIndex BoneIndex = InFootBoneIndex;
		if (BoneIndex != INDEX_NONE)
		{
			FCompactPoseBoneIndex ParentBoneIndex = RequiredBones.GetParentBoneIndex(BoneIndex);

			int32 NumIterations = NumBonesInLimb;
			while ((NumIterations-- > 0) && (ParentBoneIndex != INDEX_NONE))
			{
				const FTransform BoneTransformPS = RequiredBones.GetRefPoseTransform(BoneIndex);
				const float Extension = BoneTransformPS.GetTranslation().Size();
				OutChainLength += Extension;

				BoneIndex = ParentBoneIndex;
				ParentBoneIndex = RequiredBones.GetParentBoneIndex(BoneIndex);
			};
		}

		OutHipIndex = BoneIndex;
	}
};

void FAnimNode_FootPlacement::FindPelvisOffsetRangeForLimb(
	const UE::Anim::FootPlacement::FEvaluationContext& Context,
	const UE::Anim::FootPlacement::FLegRuntimeData& LegData,
	const FVector& InPlantTargetLocationCS,
	const FTransform& PelvisTransformCS,
	FPelvisOffsetRangeForLimb& OutPelvisOffsetRangeCS) const
{
	const UE::Anim::FootPlacement::FLegRuntimeData::FInputPoseData& LegInputPose = LegData.InputPose;
	const UE::Anim::FootPlacement::FLegRuntimeData::FBoneData& Bones = LegData.Bones;
	const float LimbLength = Bones.LimbLength;
	FVector PlantTargetLocationCS = InPlantTargetLocationCS;

	// TODO: Cache this.
	const FTransform HipToPelvis =
		LegInputPose.HipTransformCS.GetRelativeTransform(PelvisData.InputPose.FKTransformCS);
	const FTransform HipTransformCS = HipToPelvis * PelvisTransformCS;
	const FVector HipLocationCS = HipTransformCS.GetLocation();

	const FVector DesiredExtensionDelta =
		LegInputPose.FootTransformCS.GetLocation() - LegInputPose.HipTransformCS.GetLocation();

	const float DesiredExtensionSqrd = DesiredExtensionDelta.SizeSquared();
	const float DesiredExtension = FMath::Sqrt(DesiredExtensionSqrd);
	const float MaxExtension = GetMaxLimbExtension(DesiredExtension, LimbLength);

	const FPlane FootPlane = FPlane(PlantTargetLocationCS, -Context.ApproachDirCS);
	const FVector HipToPlantCS = PlantTargetLocationCS - HipLocationCS;
	const float HipHeight = FootPlane.PlaneDot(HipLocationCS);

	FVector DesiredPlantTargetLocationCS = PlantTargetLocationCS;
	FVector MaxPlantTargetLocationCS = PlantTargetLocationCS;
	// Don't do horizontal adjustments if the foot is above the hip
	if (HipHeight > 0.0f)
	{
		// Project the input pose foot and hip to the ground-aligned foot plane
		const FVector FKFootProjected = FVector::PointPlaneProject(LegInputPose.FootTransformCS.GetLocation(), FootPlane);
		const FVector HipProjected = FVector::PointPlaneProject(HipLocationCS, FootPlane);
		
		// Project both our FK and IK feet to the foot plane, and calculate distances to the projected hip.
		FVector TargetFootToHip = HipProjected - PlantTargetLocationCS;
		float TargetFootOffset = FVector::Dist(HipProjected, PlantTargetLocationCS);
		float InitialFootOffset = FVector::Dist(HipProjected, FKFootProjected);

		// Move our IK foot to our FK foot by about the foot's length horizontally.
		// This will make our heel lift before it drops the hips when locking.
		//@TODO: This is not a very accurate estimate, but neither is our foot roll yet. Fix this when we improve the foot roll.
		const FVector ToFKFoot = (FKFootProjected - PlantTargetLocationCS);
		const FVector ToFKFootDir = ToFKFoot.GetSafeNormal();
		const float ToFKDistance = ToFKFoot.Size();
		PlantTargetLocationCS = PlantTargetLocationCS + ToFKFootDir * FMath::Min(ToFKDistance, Bones.FootLength) * PelvisSettings.HeelLiftRatio;

		// Calculate new correction offset and direction
		TargetFootToHip = HipProjected - PlantTargetLocationCS;
		TargetFootOffset = FVector::Dist(HipProjected, PlantTargetLocationCS);

		auto FindPlantLocationAdjustedByOrtogonalLimit = 
			[	TargetFootOffset, TargetFootToHip,
				HipHeight, InitialFootOffset	]
			(const float LegLength, const float MaxHipOffset, const FVector& FootLocation)
		{
			FVector AdjustedPlantTargetLocationCS = FootLocation;

			// The minimum height our hip can be at after horizontal adjustments
			const float MinHeight = HipHeight - MaxHipOffset;
			const float MinHeightSqrd = FMath::Square(MinHeight);

			// Find how far our foot would be from the projected hip, if the leg was at max extension. 
			const float LegLengthSqrd = FMath::Square(LegLength);
			const float MaxFootOffset = FMath::Sqrt(FMath::Max(0.0f, LegLengthSqrd - MinHeightSqrd));

			// If the input pose is already further than this, respect the input pose
			const float MaxFootOffsetClamped = FMath::Max(InitialFootOffset, MaxFootOffset);

			if (TargetFootOffset > MaxFootOffsetClamped)
			{
				// Move the foot towards the projected hip
				AdjustedPlantTargetLocationCS += (TargetFootOffset - MaxFootOffsetClamped) * TargetFootToHip.GetSafeNormal();
			}

			return AdjustedPlantTargetLocationCS;
		};

		MaxPlantTargetLocationCS = FindPlantLocationAdjustedByOrtogonalLimit(MaxExtension, PelvisSettings.MaxOffsetHorizontal, PlantTargetLocationCS);
		DesiredPlantTargetLocationCS = FindPlantLocationAdjustedByOrtogonalLimit(DesiredExtension,  PelvisSettings.MaxOffsetHorizontal, PlantTargetLocationCS);
	}

	// Taken from http://runevision.com/thesis/rune_skovbo_johansen_thesis.pdf
	// Chapter 7.4.2
	//	Intersections are found of a vertical line going through the original hip
	//	position and two spheres with their centers at the new ankle position (PlantTargetLocationCS) 
	//	Sphere 1 has a radius of the distance between the hip and ankle in the input pose (DesiredExtension)
	//	Sphere 2 has a radius corresponding to the length of the leg from hip to ankle (MaxExtension).
	FVector MaxOffsetLocation;
	FVector DesiredOffsetLocation;
	FMath::SphereDistToLine(MaxPlantTargetLocationCS, MaxExtension, HipLocationCS - Context.ApproachDirCS * TraceSettings.EndOffset, Context.ApproachDirCS, MaxOffsetLocation);
	FMath::SphereDistToLine(DesiredPlantTargetLocationCS, DesiredExtension, HipLocationCS - Context.ApproachDirCS * TraceSettings.EndOffset, Context.ApproachDirCS, DesiredOffsetLocation);
	
	const float MaxOffset = (MaxOffsetLocation - HipLocationCS) | -Context.ApproachDirCS;
	const float DesiredOffset = (DesiredOffsetLocation - HipLocationCS) | -Context.ApproachDirCS;
	OutPelvisOffsetRangeCS.MaxExtension = MaxOffset;
	OutPelvisOffsetRangeCS.DesiredExtension = DesiredOffset;

	// Calculate min offset considering only the height of the foot
	// Poses where the foot's height is close to the hip's height are bad. 
	const float MinExtension = GetMinLimbExtension(DesiredExtension, LimbLength);
	const FVector MinOffsetLocation = DesiredPlantTargetLocationCS + -Context.ApproachDirCS * MinExtension;

	const float MinOffset = (MinOffsetLocation - HipLocationCS) | -Context.ApproachDirCS;
	// Limit pelvis compression adjustment by the height of the foot. We can always bring the foot closer to the ground in post-adjustments
	OutPelvisOffsetRangeCS.MinExtension = MinOffset - LegInputPose.DistanceToPlant;
}

float FAnimNode_FootPlacement::CalcTargetPlantPlaneDistance(
	const UE::Anim::FootPlacement::FEvaluationContext& Context,
	const UE::Anim::FootPlacement::FLegRuntimeData::FInputPoseData& LegInputPose) const
{
	const FTransform IKBallBoneCS = LegInputPose.FootToBall * LegInputPose.FootTransformCS;

	const FTransform& IKFootRootCS = PelvisData.InputPose.IKRootTransformCS;
	const FPlane IKGroundPlaneCS = 
		FPlane(	PelvisData.InputPose.IKRootTransformCS.GetLocation(), 
				PelvisData.InputPose.IKRootTransformCS.TransformVectorNoScale(FVector::UpVector));

	// TODO: I'm just getting the distance between bones and the plane, instead of actual foot/ball bases
	const TOptional<float> FootBaseDistance = 
		UE::Anim::FootPlacement::GetDistanceToPlaneAlongDirection(LegInputPose.FootTransformCS.GetLocation(), IKGroundPlaneCS, Context.ApproachDirCS);
	const TOptional<float> BallBaseDistance = 
		UE::Anim::FootPlacement::GetDistanceToPlaneAlongDirection(IKBallBoneCS.GetLocation(), IKGroundPlaneCS, Context.ApproachDirCS);

	const float PlantPlaneDistance = FMath::Min(FootBaseDistance.Get(0.0f), BallBaseDistance.Get(0.0f));
	return PlantPlaneDistance;
}


void FAnimNode_FootPlacement::AlignPlantToGround(
	const UE::Anim::FootPlacement::FEvaluationContext& Context,
	const FPlane& PlantPlaneWS,
	const UE::Anim::FootPlacement::FLegRuntimeData::FInputPoseData& LegInputPose,
	FTransform& InOutFootTransformWS,
	FQuat& OutTwistCorrection) const
{
	const FTransform InputPoseFootTransformWS = LegInputPose.FootTransformCS * Context.OwningComponentToWorld;

	// It is assumed the distance from the plane defined by ik foot root to the ik reference, along the trace 
	// direction, must remain the same.
	// TODO: This wont work well when the animation doesn't have a single plant plane, i.e. a walking upstairs anim
	const FTransform IKFootRootWS = PelvisData.InputPose.IKRootTransformCS * Context.OwningComponentToWorld;
	const FPlane IKFootRootPlaneWS = FPlane(IKFootRootWS.GetLocation(), IKFootRootWS.TransformVectorNoScale(FVector::UpVector));
	const TOptional<float> IKFootRootToFootRootTargetDistance = 
		UE::Anim::FootPlacement::GetDistanceToPlaneAlongDirection(InputPoseFootTransformWS.GetLocation(), IKFootRootPlaneWS, Context.ApproachDirWS);

	const FVector CorrectedPlaneIntersectionWS = UE::Anim::FootPlacement::PointDirectionPlaneIntersection(
		InOutFootTransformWS.GetLocation(),
		Context.ApproachDirWS,
		PlantPlaneWS);

	const FVector CorrectedLocationWS =
		CorrectedPlaneIntersectionWS - (Context.ApproachDirWS * IKFootRootToFootRootTargetDistance.Get(0.0f));

	// The relationship between the ik reference and the normal of the plane defined by the ik foot root must also be 
	// respected
	const FQuat PlanePlaneDeltaRotation = FQuat::FindBetweenNormals(IKFootRootPlaneWS.GetNormal(), PlantPlaneWS.GetNormal());
	const FQuat InputPoseAlignedRotationWS = PlanePlaneDeltaRotation * InputPoseFootTransformWS.GetRotation();

	// Find the rotation that will take us from the Aligned Input Pose to the Unaligned IK Foot 
	const FQuat UnalignedIKFootToUnalignedInputPoseRotationDelta =
		InputPoseAlignedRotationWS.Inverse() * InOutFootTransformWS.GetRotation();
	const FVector IKReferenceNormalFootSpace = InputPoseAlignedRotationWS.UnrotateVector(PlantPlaneWS.GetNormal());

	// Calculate and apply the amount of twist around the IK Root plane. 
	FQuat OutSwing;
	UnalignedIKFootToUnalignedInputPoseRotationDelta.ToSwingTwist(IKReferenceNormalFootSpace,
		OutSwing,
		OutTwistCorrection);
	const FQuat AlignedRotationWS = InputPoseAlignedRotationWS * OutTwistCorrection;

	// Find the rotation that will take us from aligned to unaligned foot
	const FQuat AlignedToUnalignedRotationDelta =
		AlignedRotationWS.Inverse() * InOutFootTransformWS.GetRotation();
	// The rotation is a delta so we won't need to re-orient this vector
	const FVector FootToBallDir = LegInputPose.FootToBall.GetTranslation().GetSafeNormal();
	FQuat AnkleTwist;
	AlignedToUnalignedRotationDelta.ToSwingTwist(FootToBallDir,
		OutSwing,
		AnkleTwist);
	// Counter the aligned ankle twist by the user-defined amount
	const FQuat TwistCorrectedRotationWS = AlignedRotationWS *  FQuat::Slerp(FQuat::Identity, AnkleTwist, PlantSettings.AnkleTwistReduction);

	// TODO: Clipping will occur due to rotation. Figure out how much we need to adjust the foot vertically to prevent clipping.

	InOutFootTransformWS = FTransform(TwistCorrectedRotationWS, CorrectedLocationWS);;
}

FTransform FAnimNode_FootPlacement::UpdatePlantOffsetInterpolation(
	const UE::Anim::FootPlacement::FEvaluationContext& Context,
	UE::Anim::FootPlacement::FLegRuntimeData::FInterpolationData& InOutInterpData) const
{
	const FVector IkBaseSpringTranslation = UKismetMathLibrary::VectorSpringInterp(
		InOutInterpData.UnalignedFootOffset.GetTranslation(), FVector::ZeroVector, InOutInterpData.PlantOffsetTranslationSpringState,
		InterpolationSettings.UnplantLinearStiffness,
		InterpolationSettings.UnplantLinearDamping,
		Context.UpdateDeltaTime, 1.0f, 0.0f);

	// Since the alignment is just a translation offset, there's no need to calculate a different offset.
	const FQuat IkBaseSpringRotation = UKismetMathLibrary::QuaternionSpringInterp(
		InOutInterpData.UnalignedFootOffset.GetRotation(), FQuat::Identity, InOutInterpData.PlantOffsetRotationSpringState,
		InterpolationSettings.UnplantAngularStiffness,
		InterpolationSettings.UnplantAngularDamping,
		Context.UpdateDeltaTime, 1.0f, 0.0f);

	const FTransform IkBaseSpringOffset = FTransform(IkBaseSpringRotation, IkBaseSpringTranslation);
	return IkBaseSpringOffset;
}

void FAnimNode_FootPlacement::UpdatePlantingPlaneInterpolation(
	const UE::Anim::FootPlacement::FEvaluationContext& Context,
	const FTransform& FootTransformWS,
	const FTransform& LastAlignedFootTransform,
	const float AlignmentAlpha,
	FPlane& InOutPlantPlane,
	const UE::Anim::FootPlacement::FLegRuntimeData::FInputPoseData& LegInputPose,
	UE::Anim::FootPlacement::FLegRuntimeData::FInterpolationData& InOutInterpData) const
{
	const FVector TraceDirection = Context.ApproachDirWS;
	const FPlane LastPlantPlane = InOutPlantPlane;

	bool bFoundGround = false;
	FVector ImpactLocationWS = FVector::ZeroVector;
	if (TraceSettings.bDisableComplexTrace)
	{
		// Trace against simple geometry only
		bFoundGround = UE::Anim::FootPlacement::FindPlantPlane(Context, TraceSettings, FootTransformWS.GetLocation(), false, InOutPlantPlane, ImpactLocationWS);
	}
	else
	{
		// Trace against complex geometry only to plant accurately
		bFoundGround = UE::Anim::FootPlacement::FindPlantPlane(Context, TraceSettings, FootTransformWS.GetLocation(), true, InOutPlantPlane, ImpactLocationWS);
	}

	if (CharacterData.bIsOnGround == false)
	{
		const FPlane GroundPlane = FPlane(ImpactLocationWS, -Context.ApproachDirWS);
		const FVector SourceGroundPoint = Context.OwningComponentToWorld.TransformPosition(PelvisData.InputPose.IKRootTransformCS.GetLocation());
		// if we're in the air, try to bring foot to source pose
		InOutPlantPlane = FPlane(SourceGroundPoint, -Context.ApproachDirWS);
	}

	if (InterpolationSettings.bEnableFloorInterpolation && !bIsFirstUpdate)
	{
		FVector CurrPlaneIntersection = UE::Anim::FootPlacement::PointDirectionPlaneIntersection(
			FootTransformWS.GetLocation(),
			TraceDirection,
			InOutPlantPlane);

		const FVector LastPlaneIntersection = UE::Anim::FootPlacement::PointDirectionPlaneIntersection(
			LastAlignedFootTransform.GetLocation(),
			TraceDirection,
			LastPlantPlane);

		const FVector PrevPlaneIntersection = UE::Anim::FootPlacement::PointDirectionPlaneIntersection(
			FootTransformWS.GetLocation(),
			TraceDirection,
			LastPlantPlane);

		const float LastPlaneDeltaZ = LastPlaneIntersection.Z - CurrPlaneIntersection.Z;
		const float PrevPlaneDeltaZ = PrevPlaneIntersection.Z - CurrPlaneIntersection.Z;
		float AdjustedPrevZ = FMath::Abs(LastPlaneDeltaZ) < FMath::Abs(PrevPlaneDeltaZ) ?
			LastPlaneIntersection.Z : PrevPlaneIntersection.Z;

		if (CharacterData.bIsOnGround)
		{
			// Since our ground plane is in root space, apply the component delta if it gets us closer to the desired ground plane.
			// This means if your ground geometry for capsule is smooth, we want to leverage that for foot alignment. 
			// When ground geometry is not smooth, we follow the same logic as pelvis interpolation, and attempt to smooth out the jump in world space.
			const float GroundPlaneDelta = CurrPlaneIntersection.Z  - AdjustedPrevZ;
			if ((GroundPlaneDelta) > 0.0f)
			{
				AdjustedPrevZ += FMath::Min(FMath::Abs(-CharacterData.ComponentMoveDeltaWS.Z), GroundPlaneDelta);
			}
			else
			{
				AdjustedPrevZ += FMath::Max(-FMath::Abs(CharacterData.ComponentMoveDeltaWS.Z), GroundPlaneDelta);
			}
		}

		float PlantPlaneSpringHeight = UKismetMathLibrary::FloatSpringInterp(
			AdjustedPrevZ, CurrPlaneIntersection.Z, InOutInterpData.GroundHeightSpringState,
			InterpolationSettings.FloorLinearStiffness,
			InterpolationSettings.FloorLinearDamping,
			Context.UpdateDeltaTime, 1.0f, 0.0f);

		CurrPlaneIntersection.Z = PlantPlaneSpringHeight;

		if ((TraceSettings.MaxGroundPenetration >= 0.0f) && bFoundGround)
		{
			// Prevent the foot from clipping too much into geometry due to interpolation
			const FPlane GroundPlane = FPlane(ImpactLocationWS, InOutPlantPlane.GetNormal());
			const TOptional<float> DistanceToGroundPlane =
				UE::Anim::FootPlacement::GetDistanceToPlaneAlongDirection(CurrPlaneIntersection, GroundPlane, Context.ApproachDirCS);
			const float PenetrationAmount = -DistanceToGroundPlane.Get(0.0f) - TraceSettings.MaxGroundPenetration;
			if (PenetrationAmount > 0.0f)
			{
				CurrPlaneIntersection -= Context.ApproachDirCS * PenetrationAmount;
			}
		}

		FQuat FloorNormalRotation = FQuat::FindBetweenNormals(
			LastPlantPlane.GetNormal(), InOutPlantPlane.GetNormal());
		const FQuat FloorSpringNormalRotation = UKismetMathLibrary::QuaternionSpringInterp(
			FQuat::Identity, FloorNormalRotation, InOutInterpData.GroundRotationSpringState,
			InterpolationSettings.FloorAngularStiffness,
			InterpolationSettings.FloorAngularDamping,
			Context.UpdateDeltaTime, 1.0f, 0.0f);

		const FVector PlantPlaneSpringNormal = FloorSpringNormalRotation.RotateVector(LastPlantPlane.GetNormal());
		const FPlane PlantingPlane = FPlane(CurrPlaneIntersection, PlantPlaneSpringNormal);

		InOutPlantPlane = PlantingPlane;
	}
}

void FAnimNode_FootPlacement::DeterminePlantType(
	const UE::Anim::FootPlacement::FEvaluationContext& Context,
	const FTransform& FKTransformWS,
	const FTransform& CurrentBoneTransformWS,
	UE::Anim::FootPlacement::FLegRuntimeData::FPlantData& InOutPlantData,
	const UE::Anim::FootPlacement::FLegRuntimeData::FInputPoseData& LegInputPose) const
{
	using namespace UE::Anim::FootPlacement;

	const bool bWasPlanted = InOutPlantData.PlantType != EPlantType::Unplanted;
	const bool bWantedToPlant = InOutPlantData.bWantsToPlant;

	InOutPlantData.bWantsToPlant = WantsToPlant(Context, LegInputPose);
	InOutPlantData.PlantType = EPlantType::Unplanted;

	if (!InOutPlantData.bWantsToPlant)
	{
		return;
	}

	// Test for un-plant
	if (bWasPlanted)
	{
		const FVector PlantTranslationWS =
			CurrentBoneTransformWS.GetLocation() - FKTransformWS.GetLocation();

		// TODO: Test along approach direction
		// Don't consider the limits to be exceeded if replant radius == unplant radius.
		const bool bPlantTranslationExceeded =
			PlantSettings.ReplantRadiusRatio < 1.0f &&
			PlantTranslationWS.SizeSquared2D() > PlantRuntimeSettings.UnplantRadiusSqrd;
		const bool bPlantRotationExceeded =
			PlantSettings.ReplantAngleRatio < 1.0f &&
			FMath::Abs(InOutPlantData.TwistCorrection.W) <
			PlantRuntimeSettings.CosHalfUnplantAngle;

		if (!bPlantTranslationExceeded && !bPlantRotationExceeded)
		{
			// Carry over result from last plant.
			InOutPlantData.PlantType = InOutPlantData.LastPlantType;
		}
	}
	else if (!bWantedToPlant)
	{
		// If FK wasn't planted last frame, and it is on this frame, we're planted 
		InOutPlantData.PlantType = UE::Anim::FootPlacement::EPlantType::Planted;
	}
	else // Test for re-plant
	{
		const FVector PlantLocationDelta =
			CurrentBoneTransformWS.GetLocation() - FKTransformWS.GetLocation();

		// TODO: Test along approach direction
		const float LocationDeltaSizeSqrd = PlantLocationDelta.SizeSquared2D();

		const bool bLocationWithinBounds =
			LocationDeltaSizeSqrd <= PlantRuntimeSettings.ReplantRadiusSqrd;
		const bool bTwistWithinBounds =
			FMath::Abs(InOutPlantData.TwistCorrection.W) >=
			PlantRuntimeSettings.CosHalfReplantAngle;

		if (bLocationWithinBounds && bTwistWithinBounds)
		{
			InOutPlantData.PlantType = UE::Anim::FootPlacement::EPlantType::Replanted;
		}
	}
}

float FAnimNode_FootPlacement::GetMaxLimbExtension(const float DesiredExtension, const float LimbLength) const
{
	if (DesiredExtension > LimbLength)
	{
		return DesiredExtension;
	}

	const float RemainingLength = LimbLength - DesiredExtension;
	return DesiredExtension + RemainingLength * PlantSettings.MaxExtensionRatio;
}

float FAnimNode_FootPlacement::GetMinLimbExtension(const float DesiredExtension, const float LimbLength) const
{
	return FMath::Min(DesiredExtension, LimbLength * PlantSettings.MinExtensionRatio);
}

void FAnimNode_FootPlacement::ResetRuntimeData() 
{
	PelvisData.Interpolation = UE::Anim::FootPlacement::FPelvisRuntimeData::FInterpolationData();

	LegsData.Reset();
	LegsData.SetNumUninitialized(LegDefinitions.Num());

	for (int32 LegIndex = 0; LegIndex < LegsData.Num(); ++LegIndex)
	{
		UE::Anim::FootPlacement::FLegRuntimeData& LegData = LegsData[LegIndex];
		LegData.Idx = LegIndex;
		LegData.Interpolation = UE::Anim::FootPlacement::FLegRuntimeData::FInterpolationData();
	}

#if ENABLE_FOOTPLACEMENT_DEBUG
	DebugData.Init(LegDefinitions.Num());
#endif

	bIsFirstUpdate = true;
}

bool FAnimNode_FootPlacement::WantsToPlant(
	const UE::Anim::FootPlacement::FEvaluationContext& Context,
	const UE::Anim::FootPlacement::FLegRuntimeData::FInputPoseData& LegInputPose) const
{
#if ENABLE_FOOTPLACEMENT_DEBUG
	if (!CVarAnimNodeFootPlacementEnableLock.GetValueOnAnyThread())
	{
		return false;
	}
#endif

	if ((PlantSettings.LockType == EFootPlacementLockType::Unlocked) || FMath::IsNearlyZero(LegInputPose.LockAlpha))
	{
		return false;
	}

	const bool bPassesPlantDistanceCheck = LegInputPose.DistanceToPlant < PlantSettings.DistanceToGround;
	const bool bPassesSpeedCheck = LegInputPose.Speed < PlantSettings.SpeedThreshold;
	return bPassesPlantDistanceCheck && bPassesSpeedCheck;
}

float FAnimNode_FootPlacement::GetAlignmentAlpha(
	const UE::Anim::FootPlacement::FEvaluationContext& Context,
	const UE::Anim::FootPlacement::FLegRuntimeData::FInputPoseData& LegInputPose) const
{
	return FMath::Clamp(
		FMath::GetRangePct(FVector2D(PlantSettings.UnalignmentSpeedThreshold,
			PlantSettings.SpeedThreshold),
			LegInputPose.Speed),
		0.0f, 1.0f);
}

FTransform FAnimNode_FootPlacement::GetFootPivotAroundBallWS(
	const UE::Anim::FootPlacement::FEvaluationContext& Context,
	const UE::Anim::FootPlacement::FLegRuntimeData::FInputPoseData& LegInputPose,
	const FTransform& LastPlantTransformWS) const
{
	const FTransform BallTransformWS = LegInputPose.BallTransformCS * Context.OwningComponentToWorld;

	const FTransform PinnedBallTransformWS = FTransform(
		BallTransformWS.GetRotation(),
		(LegInputPose.FootToBall * LastPlantTransformWS).GetLocation(),
		BallTransformWS.GetScale3D());

	return LegInputPose.BallToFoot * PinnedBallTransformWS;

}

UE::Anim::FootPlacement::FPlantResult FAnimNode_FootPlacement::FinalizeFootAlignment(
	const UE::Anim::FootPlacement::FEvaluationContext& Context,
	UE::Anim::FootPlacement::FLegRuntimeData& LegData,
	const FFootPlacemenLegDefinition& LegDef,
	const FTransform& PelvisTransformCS)
{
	// TODO: Cache this value
	const FTransform& FKPelvisToHipCS =
		LegData.InputPose.HipTransformCS.GetRelativeTransform(PelvisData.InputPose.FKTransformCS);
	const FTransform FinalHipTransformCS = FKPelvisToHipCS * PelvisTransformCS;
	FTransform CorrectedFootTransformCS = LegData.AlignedFootTransformRS * GetRootToComponent();
	const FVector CorrectedBallLocationCS = (LegData.InputPose.FootToBall * CorrectedFootTransformCS).GetLocation();

	// avoid hyper extension - start
	const FVector InputPoseHipToFoot =
		LegData.InputPose.FootTransformCS.GetLocation() - LegData.InputPose.HipTransformCS.GetLocation();
	const FVector InputPoseHipToFootDir = InputPoseHipToFoot.GetSafeNormal();
	const FVector CorrectedHipToFootDir =
		(CorrectedFootTransformCS.GetLocation() - FinalHipTransformCS.GetLocation()).GetSafeNormal();

	if (!InputPoseHipToFoot.IsNearlyZero() && !CorrectedHipToFootDir.IsNearlyZero())
	{
		{
			const float DesiredExtension = InputPoseHipToFoot.Size();
			const float MaxExtension = GetMaxLimbExtension(DesiredExtension, LegData.Bones.LimbLength);
			const float CurrentExtension =
				FVector::Dist(CorrectedFootTransformCS.GetLocation(), FinalHipTransformCS.GetLocation());

			const float HyperExtensionAmount = CurrentExtension - MaxExtension;
			float HyperExtensionRemaining = HyperExtensionAmount;

			if (CurrentExtension > MaxExtension)
			{
				const bool bIsPlanted = LegData.Plant.PlantType != UE::Anim::FootPlacement::EPlantType::Unplanted;
				const bool bWasPlanted = LegData.Plant.LastPlantType != UE::Anim::FootPlacement::EPlantType::Unplanted;
				const bool bPlantedThisFrame = bIsPlanted && !bWasPlanted;

				if (!bIsPlanted)
				{
					// If there's any overextension and we're unplanted, target is unreachable
					// Don't plant until we're in re-plant range
					LegData.Plant.bCanReachTarget = false;
				}

				const bool bRecentlyUnplanted = !bIsPlanted && LegData.Plant.TimeSinceFullyUnaligned == 0.0f;
				// Try to keep the tip on spot if we're unplanting
				// Don't do this until we've reached the plant target once
				const bool bCanLiftHeel = bRecentlyUnplanted || (bIsPlanted && LegData.Plant.bCanReachTarget) || PlantSettings.bAdjustHeelBeforePlanting;
				if (bCanLiftHeel)
				{
					// Scale this value by our FK transition alpha to not pop
					const float MaxPullTowardsHip =  FMath::Min(LegData.Bones.FootLength, HyperExtensionRemaining) * LegData.InputPose.AlignmentAlpha;
					HyperExtensionRemaining -= MaxPullTowardsHip;

					//@TODO: This isn't accurate. Roll the foot around around the toes, and counter-adjust toe rotation instead
					const FVector CorrectedFootLocationCS = CorrectedFootTransformCS.GetLocation() - MaxPullTowardsHip * CorrectedHipToFootDir;

					// Rotate the foot to keep the toe at the same spot
					const FVector InitialFootToToe = CorrectedBallLocationCS - CorrectedFootTransformCS.GetLocation();
					const FVector CorrectedFootToToe = CorrectedBallLocationCS - CorrectedFootLocationCS;
					FQuat DeltaSlopeRotation = FQuat::FindBetweenVectors(InitialFootToToe, CorrectedFootToToe);

#if ENABLE_FOOTPLACEMENT_DEBUG
					FRotator DeltaSlopeRotator = DeltaSlopeRotation.Rotator();
#endif

					// Rotate the foot to preserve the ball's location
					CorrectedFootTransformCS.SetRotation(DeltaSlopeRotation * CorrectedFootTransformCS.GetRotation());
					CorrectedFootTransformCS.NormalizeRotation();

					// Move the foot bone closer to the hip prevent overextension
					CorrectedFootTransformCS.SetLocation(CorrectedFootLocationCS);
					check(!CorrectedFootTransformCS.ContainsNaN());
				}

				// Fix any remaining hyper-extension
				if (HyperExtensionRemaining > 0.0f)
				{
					// Move IK bone towards the hip bone.
					// TODO: Pull towards the FK bone? This pull lifts the foot from the ground and it might be 
					// preferable to slide. This causes discontinuities when the foot is no longer hyper-extended
					FVector NotHyperextendedPlantLocation;
					FMath::SphereDistToLine(FinalHipTransformCS.GetLocation(), MaxExtension, CorrectedFootTransformCS.GetLocation(), CorrectedHipToFootDir, NotHyperextendedPlantLocation);
					CorrectedFootTransformCS.SetLocation(NotHyperextendedPlantLocation);
					check(!CorrectedFootTransformCS.ContainsNaN());
				}
			}
			else
			{
				// No overextension, therefore we can reach the target, and can do future tip/ball adjustments
				LegData.Plant.bCanReachTarget = true;
			}

#if ENABLE_FOOTPLACEMENT_DEBUG
			DebugData.LegsInfo[LegData.Idx].HyperExtensionAmount = HyperExtensionAmount;
			DebugData.LegsInfo[LegData.Idx].RollAmount = HyperExtensionAmount - HyperExtensionRemaining;
			DebugData.LegsInfo[LegData.Idx].PullAmount = FMath::Max(0.0f, HyperExtensionRemaining);
#endif // (ENABLE_FOOTPLACEMENT_DEBUG)
		}
	}

	// Next the plant is ajusted to prevent penetration with the planting plane. To do that, first the base of the plant
	// and the tip must be calculated (note that because the ground plane interpolates, this does not prevent physical penetration 
	// with the geometry).
	// TODO: Consolidate with CalcTargetPlantPlaneDistance
	const FPlane PlantPlaneCS = LegData.Plant.GetPlantPlaneCS(GetRootToComponent());
	const TOptional<float> FootDistance = UE::Anim::FootPlacement::GetDistanceToPlaneAlongDirection(
		CorrectedBallLocationCS,
		PlantPlaneCS,
		Context.ApproachDirCS);
	const TOptional<float> BallDistance = UE::Anim::FootPlacement::GetDistanceToPlaneAlongDirection(
		CorrectedFootTransformCS.GetLocation(),
		PlantPlaneCS,
		Context.ApproachDirCS);
	float MinDistance = FMath::Min(FootDistance.Get(0.0f), BallDistance.Get(0.0f));
	// Allow as much penetration as the source animation.
	MinDistance -= FMath::Min(0.0f, LegData.InputPose.DistanceToPlant);
	// A min distance < 0.0f means there was penetration
	if (MinDistance < 0.0f)
	{
		CorrectedFootTransformCS.AddToTranslation(MinDistance * Context.ApproachDirCS);
	}
	check(!CorrectedFootTransformCS.ContainsNaN());

	// Fix any remaining hyper-compression. Clip into the ground plane if necessary.
	// Doing this after pushing the feet out of the ground plane ensures we won't end up in awkward poses. 
	{
		FVector NotHyperextendedPlantLocation;
		const float MinExtension = GetMinLimbExtension(FMath::Abs(InputPoseHipToFoot | Context.ApproachDirCS), LegData.Bones.LimbLength);

		// Offset our hip plane by min extension
		FPlane HipPlane = FPlane(FinalHipTransformCS.GetLocation() + Context.ApproachDirCS * MinExtension, Context.ApproachDirCS);
		const float DistanceToHipPlane = HipPlane.PlaneDot(CorrectedFootTransformCS.GetLocation());

		if (DistanceToHipPlane < 0.0f)
		{
			// Move foot to hip plane if we're past it.
			NotHyperextendedPlantLocation = CorrectedFootTransformCS.GetLocation() - Context.ApproachDirCS * DistanceToHipPlane;
			CorrectedFootTransformCS.SetLocation(NotHyperextendedPlantLocation);
		}
	}

	if (LegData.InputPose.DisableLeg > 0.0f)
	{
		FTransform DisabledLegTransform = LegData.InputPose.FootFKTransformCS;
		DisabledLegTransform.SetTranslation(DisabledLegTransform.GetTranslation() + PelvisData.Interpolation.PelvisTranslationOffset * (1.0-PelvisData.DisablePelvis));
		CorrectedFootTransformCS.BlendWith(DisabledLegTransform, LegData.InputPose.DisableLeg);
	}

	check(!CorrectedFootTransformCS.ContainsNaN());

	// TODO: Do adjustments to the ball and hip
	UE::Anim::FootPlacement::FPlantResult Result =
	{
		{ LegData.Bones.IKIndex, CorrectedFootTransformCS },
		// { LegData.Bones.BallIndex, CorrectedBallTransformCS },
		//{ LegData.Bones.HipIndex, CorrectedHipTransformCS }
	};

	return Result;
}

#if ENABLE_FOOTPLACEMENT_DEBUG
void FAnimNode_FootPlacement::DrawVLog(
	const UE::Anim::FootPlacement::FEvaluationContext& Context,
	const UE::Anim::FootPlacement::FLegRuntimeData& LegData,
	const UE::Anim::FootPlacement::FPlantResult& PlantResult) const
{
	using namespace UE::Anim::FootPlacement;

	const FColor FKColor = FColor::Blue;
	const FColor PlantedColor = FColor::Red;
	const FColor UnplantedColor = FColor::Green;
	const FColor ReplantedColor = FColor::Orange;

	FColor CurrentPlantColor;
	switch (LegData.Plant.PlantType)
	{
	case UE::Anim::FootPlacement::EPlantType::Planted: CurrentPlantColor = PlantedColor; break;
	case UE::Anim::FootPlacement::EPlantType::Unplanted: CurrentPlantColor = UnplantedColor; break;
	case UE::Anim::FootPlacement::EPlantType::Replanted: CurrentPlantColor = ReplantedColor; break;
	default: check(false); break; //not implemented
	}

	const FTransform FKBoneTransformWS =
		LegData.InputPose.FootToGround *
		LegData.InputPose.FootTransformCS *
		Context.OwningComponentToWorld;

	const FTransform IKBoneTransformWS =
		LegData.InputPose.FootToGround *
		LegData.AlignedFootTransformWS;

	const FPlane PlantPlaneWS = LegData.Plant.GetPlantPlaneWS(GetRootToComponent(), Context.OwningComponentToWorld);

	const FVector FKBoneLocationProjectedWS = UE::Anim::FootPlacement::PointDirectionPlaneIntersection(
		FKBoneTransformWS.GetLocation(),
		Context.ApproachDirWS,
		PlantPlaneWS);
	
	UObject* LogOwner = Context.CSPContext.AnimInstanceProxy->GetAnimInstanceObject();
	FName LogCategory = "FootPlacement";

	UE_VLOG_SPHERE(LogOwner, LogCategory, Display, FKBoneTransformWS.GetLocation(), 0, FKColor, TEXT(""));
	
	UE_VLOG_SPHERE(LogOwner, LogCategory, Display, FKBoneLocationProjectedWS, 0, FKColor, TEXT(""));

	UE_VLOG_SPHERE(LogOwner, LogCategory, Display, IKBoneTransformWS.GetLocation(), 0, CurrentPlantColor, TEXT(""));

	const FVector IKBoneLocationProjectedWS = UE::Anim::FootPlacement::PointDirectionPlaneIntersection(
		IKBoneTransformWS.GetLocation(),
		Context.ApproachDirWS,
		PlantPlaneWS);

	UE_VLOG_SPHERE(LogOwner, LogCategory, Display, IKBoneLocationProjectedWS, 0, CurrentPlantColor, TEXT(""));
	UE_VLOG_SEGMENT_THICK(LogOwner, LogCategory, Display, IKBoneTransformWS.GetLocation(), IKBoneLocationProjectedWS, CurrentPlantColor, 2, TEXT(""));

	const float UnplantRadius = PlantSettings.UnplantRadius;
	const FVector PlantCenter = UE::Anim::FootPlacement::PointDirectionPlaneIntersection(
		IKBoneTransformWS.GetLocation(),
		Context.ApproachDirWS,
		PlantPlaneWS);

	UE_VLOG_CIRCLE(LogOwner, LogCategory, Display, PlantCenter, PlantPlaneWS.GetNormal(), UnplantRadius, PlantedColor, TEXT(""));
	
	if (PlantSettings.ReplantRadiusRatio < 1.0f)
	{
		const float ReplantRadius =
			PlantSettings.UnplantRadius *
			PlantSettings.ReplantRadiusRatio;
		
		UE_VLOG_CIRCLE(LogOwner, LogCategory, Display, PlantCenter, PlantPlaneWS.GetNormal(), ReplantRadius, ReplantedColor, TEXT(""));
	}

	// FString InputPoseMessage = FString::Printf(
	// 	TEXT("%s\n\t - InputPose [ AlignmentAlpha = %.2f, Speed = %.2f, DistanceToPlant = %.2f]"), 
	// 		*LegDefinitions[LegData.Idx].FKFootBone.BoneName.ToString(),
	// 		LegData.InputPose.AlignmentAlpha,
	// 		LegData.InputPose.Speed,
	// 		LegData.InputPose.DistanceToPlant );
	// Context.CSPContext.AnimInstanceProxy->AnimDrawDebugOnScreenMessage(InputPoseMessage, FColor::White);
	//
	// FString ExtensionMessage = FString::Printf(
	// 	TEXT("\t - HyperExtension[ Amount = %.2f, Roll = %.2f, Pull %.2f]"),
	// 		DebugData.LegsInfo[LegData.Idx].HyperExtensionAmount,
	// 		DebugData.LegsInfo[LegData.Idx].RollAmount,
	// 		DebugData.LegsInfo[LegData.Idx].PullAmount);
	// Context.CSPContext.AnimInstanceProxy->AnimDrawDebugOnScreenMessage(ExtensionMessage,
	// 	(DebugData.LegsInfo[LegData.Idx].HyperExtensionAmount <= 0.0f) ? FColor::Green : FColor::Red);

	if (PlantSettings.SeparatingDistance > 0.0f)
	{
		// FString SeparationPlaneMessage = FString::Printf(
		// 	TEXT("\t - Distance To Separating Plane = %.2f"),
		// 	DebugData.LegsInfo[LegData.Idx].DistanceToSeparatingPlane);
		// Context.CSPContext.AnimInstanceProxy->AnimDrawDebugOnScreenMessage(SeparationPlaneMessage,
		// 	(DebugData.LegsInfo[LegData.Idx].DistanceToSeparatingPlane < 0.0f) ? FColor::Red : FColor::Green);

		TRACE_ANIM_NODE_VALUE(Context.CSPContext, TEXT("DistanceToSeparatingPlane"), DebugData.LegsInfo[LegData.Idx].DistanceToSeparatingPlane);
	}

	TRACE_ANIM_NODE_VALUE(Context.CSPContext, TStringBuilder<256>().Append("HyperExtension - ").Append(FString::FromInt(LegData.Idx)).ToString(), DebugData.LegsInfo[LegData.Idx].HyperExtensionAmount);
	TRACE_ANIM_NODE_VALUE(Context.CSPContext, TStringBuilder<256>().Append("Roll - ").Append(FString::FromInt(LegData.Idx)).ToString(), DebugData.LegsInfo[LegData.Idx].RollAmount);
	TRACE_ANIM_NODE_VALUE(Context.CSPContext, TStringBuilder<256>().Append("Pull - ").Append(FString::FromInt(LegData.Idx)).ToString(), DebugData.LegsInfo[LegData.Idx].PullAmount);
	TRACE_ANIM_NODE_VALUE(Context.CSPContext, TStringBuilder<256>().Append("AlignmentAlpha - ").Append(FString::FromInt(LegData.Idx)).ToString(), LegData.InputPose.AlignmentAlpha);
	
	TRACE_ANIM_NODE_VALUE(Context.CSPContext, TStringBuilder<256>().Append("FootSpeed - ").Append(FString::FromInt(LegData.Idx)).ToString(), LegData.InputPose.Speed);
	TRACE_ANIM_NODE_VALUE(Context.CSPContext, TStringBuilder<256>().Append("DistanceToPlant - ").Append(FString::FromInt(LegData.Idx)).ToString(), LegData.InputPose.DistanceToPlant);
	TRACE_ANIM_NODE_VALUE(Context.CSPContext, TStringBuilder<256>().Append("DisableLeg - ").Append(FString::FromInt(LegData.Idx)).ToString(), LegData.InputPose.DisableLeg);

}
#endif


#if ENABLE_FOOTPLACEMENT_DEBUG
void FAnimNode_FootPlacement::DrawDebug(
	const UE::Anim::FootPlacement::FEvaluationContext& Context,
	const UE::Anim::FootPlacement::FLegRuntimeData& LegData,
	const UE::Anim::FootPlacement::FPlantResult& PlantResult) const
{
	using namespace UE::Anim::FootPlacement;

	const FColor FKColor = FColor::Blue;
	const FColor PlantedColor = FColor::Red;
	const FColor UnplantedColor = FColor::Green;
	const FColor ReplantedColor = FColor::Orange;

	FColor CurrentPlantColor;
	switch (LegData.Plant.PlantType)
	{
	case UE::Anim::FootPlacement::EPlantType::Planted: CurrentPlantColor = PlantedColor; break;
	case UE::Anim::FootPlacement::EPlantType::Unplanted: CurrentPlantColor = UnplantedColor; break;
	case UE::Anim::FootPlacement::EPlantType::Replanted: CurrentPlantColor = ReplantedColor; break;
	default: check(false); break; //not implemented
	}

	const FTransform FKBoneTransformWS =
		LegData.InputPose.FootToGround *
		LegData.InputPose.FootTransformCS *
		Context.OwningComponentToWorld;

	const FTransform IKBoneTransformWS =
		LegData.InputPose.FootToGround *
		LegData.AlignedFootTransformWS;

	const FPlane PlantPlaneWS = LegData.Plant.GetPlantPlaneWS(GetRootToComponent(), Context.OwningComponentToWorld);

	const FVector FKBoneLocationProjectedWS = UE::Anim::FootPlacement::PointDirectionPlaneIntersection(
		FKBoneTransformWS.GetLocation(),
		Context.ApproachDirWS,
		PlantPlaneWS);

	Context.CSPContext.AnimInstanceProxy->AnimDrawDebugPoint(
		FKBoneTransformWS.GetLocation(), 10.0f, FKColor, false, -1.0f, SDPG_Foreground);
	Context.CSPContext.AnimInstanceProxy->AnimDrawDebugPoint(
		FKBoneLocationProjectedWS, 15.0f, FKColor, false, -1.0f, SDPG_Foreground);
	Context.CSPContext.AnimInstanceProxy->AnimDrawDebugLine(
		FKBoneTransformWS.GetLocation(), FKBoneLocationProjectedWS,
		FKColor,
		false, -1.0f, 1.0f, SDPG_Foreground);
	Context.CSPContext.AnimInstanceProxy->AnimDrawDebugPoint(
		IKBoneTransformWS.GetLocation(), 10.0f, CurrentPlantColor, false, -1.0f, SDPG_Foreground);

	const FVector IKBoneLocationProjectedWS = UE::Anim::FootPlacement::PointDirectionPlaneIntersection(
		IKBoneTransformWS.GetLocation(),
		Context.ApproachDirWS,
		PlantPlaneWS);

	Context.CSPContext.AnimInstanceProxy->AnimDrawDebugPoint(
		IKBoneLocationProjectedWS, 15.0f, CurrentPlantColor, false, -1.0f, SDPG_Foreground);
	Context.CSPContext.AnimInstanceProxy->AnimDrawDebugLine(
		IKBoneTransformWS.GetLocation(), IKBoneLocationProjectedWS,
		CurrentPlantColor,
		false, -1.0f, 1.0f, SDPG_Foreground);

	const float UnplantRadius = PlantSettings.UnplantRadius;
	const FVector PlantCenter = UE::Anim::FootPlacement::PointDirectionPlaneIntersection(
		IKBoneTransformWS.GetLocation(),
		Context.ApproachDirWS,
		PlantPlaneWS);
	Context.CSPContext.AnimInstanceProxy->AnimDrawDebugCircle(
		PlantCenter, UnplantRadius, 24, PlantedColor,
		PlantPlaneWS.GetNormal(), false, -1.0f, SDPG_Foreground, 0.5f);

	if (PlantSettings.ReplantRadiusRatio < 1.0f)
	{
		const float ReplantRadius =
			PlantSettings.UnplantRadius *
			PlantSettings.ReplantRadiusRatio;
		Context.CSPContext.AnimInstanceProxy->AnimDrawDebugCircle(
			PlantCenter, ReplantRadius, 24, ReplantedColor,
			PlantPlaneWS.GetNormal(), false, -1.0f, SDPG_Foreground, 0.5f);
	}

	FString InputPoseMessage = FString::Printf(
		TEXT("%s\n\t - InputPose [ AlignmentAlpha = %.2f, Speed = %.2f, DistanceToPlant = %.2f]"), 
			*LegDefinitions[LegData.Idx].FKFootBone.BoneName.ToString(),
			LegData.InputPose.AlignmentAlpha,
			LegData.InputPose.Speed,
			LegData.InputPose.DistanceToPlant );
	Context.CSPContext.AnimInstanceProxy->AnimDrawDebugOnScreenMessage(InputPoseMessage, FColor::White);
	
	FString ExtensionMessage = FString::Printf(
		TEXT("\t - HyperExtension[ Amount = %.2f, Roll = %.2f, Pull %.2f]"),
			DebugData.LegsInfo[LegData.Idx].HyperExtensionAmount,
			DebugData.LegsInfo[LegData.Idx].RollAmount,
			DebugData.LegsInfo[LegData.Idx].PullAmount);
	Context.CSPContext.AnimInstanceProxy->AnimDrawDebugOnScreenMessage(ExtensionMessage,
		(DebugData.LegsInfo[LegData.Idx].HyperExtensionAmount <= 0.0f) ? FColor::Green : FColor::Red);

	if (PlantSettings.SeparatingDistance > 0.0f)
	{
		FString SeparationPlaneMessage = FString::Printf(
			TEXT("\t - Distance To Separating Plane = %.2f"),
			DebugData.LegsInfo[LegData.Idx].DistanceToSeparatingPlane);
		Context.CSPContext.AnimInstanceProxy->AnimDrawDebugOnScreenMessage(SeparationPlaneMessage,
			(DebugData.LegsInfo[LegData.Idx].DistanceToSeparatingPlane < 0.0f) ? FColor::Red : FColor::Green);

	}
}
#endif

FAnimNode_FootPlacement::FAnimNode_FootPlacement()
{
}

// TODO: implement 
void FAnimNode_FootPlacement::GatherDebugData(FNodeDebugData& NodeDebugData)
{
	ComponentPose.GatherDebugData(NodeDebugData);
}

void FAnimNode_FootPlacement::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	FAnimNode_SkeletalControlBase::Initialize_AnyThread(Context);
	ResetRuntimeData();
}

void FAnimNode_FootPlacement::UpdateInternal(const FAnimationUpdateContext& Context)
{
	FAnimNode_SkeletalControlBase::UpdateInternal(Context);

	// If we just became relevant and haven't been initialized yet, then reinitialize foot placement.
	if (!bIsFirstUpdate && UpdateCounter.HasEverBeenUpdated() && !UpdateCounter.WasSynchronizedCounter(Context.AnimInstanceProxy->GetUpdateCounter()))
	{
		ResetRuntimeData();
	}
	UpdateCounter.SynchronizeWith(Context.AnimInstanceProxy->GetUpdateCounter());

	CachedDeltaTime += Context.GetDeltaTime();
}

void FAnimNode_FootPlacement::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output,
	TArray<FBoneTransform>& OutBoneTransforms)
{
	SCOPE_CYCLE_COUNTER(STAT_FootPlacement_Eval);

	check(OutBoneTransforms.Num() == 0);

#if WITH_EDITOR
	// Early out if we are in an editor world and not updating animation to avoid generating invalid data
	USkeletalMeshComponent* SMC = Output.AnimInstanceProxy->GetSkelMeshComponent();
	bool bIsGameOrEditorPreviewWorld = SMC->GetWorld()->IsGameWorld() || SMC->GetWorld()->WorldType == EWorldType::EditorPreview;
	if (!bIsGameOrEditorPreviewWorld && !SMC->GetUpdateAnimationInEditor())
	{
		return;
	}
#endif

	// Manually calculate distance instead of using the teleport flag to properly handle cases like crouch
	// and instantaneous root offsets i.e. when entering/leaving a vehicle.
	// See FAnimNode_Inertialization::Evaluate_AnyThread and/or UE-78594
	const float TeleportDistanceThreshold = Output.AnimInstanceProxy->GetSkelMeshComponent()->GetTeleportDistanceThreshold();
	if (!bIsFirstUpdate && (TeleportDistanceThreshold > 0.0f))
	{
		const FTransform ComponentTransform = Output.AnimInstanceProxy->GetComponentTransform();
		const FVector RootWorldSpaceLocation = ComponentTransform.TransformPosition(
			Output.Pose.GetComponentSpaceTransform(FCompactPoseBoneIndex(0)).GetTranslation());
		const FVector PrevRootWorldSpaceLocation = CharacterData.ComponentTransformWS.TransformPosition(GetRootToComponent().GetTranslation());
		if (FVector::DistSquared(RootWorldSpaceLocation, PrevRootWorldSpaceLocation) > FMath::Square(TeleportDistanceThreshold))
		{
			ResetRuntimeData();
		}
	}
	
#if ENABLE_FOOTPLACEMENT_DEBUG
	UE::Anim::FootPlacement::FDebugData LastDebugData = DebugData;
#endif

	// TODO: Support a different approach direction
	const FVector ApproachDirCS = -FVector::UpVector;
	UE::Anim::FootPlacement::FEvaluationContext FootPlacementContext(Output, ApproachDirCS, CachedDeltaTime);

	// Gather data from pose and property inputs, and do minimal processing for commonly used values
	GatherPelvisDataFromInputs(FootPlacementContext);

	for (int32 FootIndex = 0; FootIndex < LegsData.Num(); ++FootIndex)
	{
		UE::Anim::FootPlacement::FLegRuntimeData& LegData = LegsData[FootIndex];
		const FFootPlacemenLegDefinition& LegDef = LegDefinitions[FootIndex];
		GatherLegDataFromInputs(FootPlacementContext, LegData, LegDef);

		// TODO: All of these can be calculated on initialize, but in case there's value in changing these dynamically,
		// will keep this for now. If needed, change to lazy update.
		PlantRuntimeSettings.UnplantRadiusSqrd = PlantSettings.UnplantRadius * PlantSettings.UnplantRadius;
		PlantRuntimeSettings.ReplantRadiusSqrd =
			PlantRuntimeSettings.UnplantRadiusSqrd *
			PlantSettings.ReplantRadiusRatio * PlantSettings.ReplantRadiusRatio;
		PlantRuntimeSettings.CosHalfUnplantAngle = FMath::Cos(FMath::DegreesToRadians(PlantSettings.UnplantAngle / 2.0f));
		PlantRuntimeSettings.CosHalfReplantAngle
			= FMath::Cos(FMath::DegreesToRadians(
				(PlantSettings.UnplantAngle * PlantSettings.ReplantAngleRatio) / 2.0f));
	}

	ProcessCharacterState(FootPlacementContext);
	CalculateFootMidpoint(FootPlacementContext, LegsData, PelvisData.InputPose.FootMidpointCS);
	for (UE::Anim::FootPlacement::FLegRuntimeData& LegData : LegsData)
	{
		ProcessFootAlignment(FootPlacementContext, LegData);
	}

	// Based on the ground alignment, search for the best Pelvis transform
	FTransform PelvisTransformCS = SolvePelvis(FootPlacementContext);

#if ENABLE_FOOTPLACEMENT_DEBUG
	const FTransform PelvisTargetTransformCS = PelvisTransformCS;
	if (CVarAnimNodeFootPlacementDebug.GetValueOnAnyThread())
	{
		// Debug text header
		FString HeaderMessage = FString::Printf(TEXT("FOOT PLACEMENT DEBUG"));
		FootPlacementContext.CSPContext.AnimInstanceProxy->AnimDrawDebugOnScreenMessage(HeaderMessage, FColor::Cyan);
	}
#endif
	if (PelvisSettings.bEnableInterpolation)
	{
		const FTransform& RootToComponent = GetRootToComponent();
		const FTransform TargetPelvisTransformRS = PelvisTransformCS.GetRelativeTransform(RootToComponent);
		const FTransform PelvisTransformRS = UpdatePelvisInterpolationRootSpace(FootPlacementContext, TargetPelvisTransformRS);
		PelvisTransformCS = PelvisTransformRS * RootToComponent;
	}
	
	PelvisData.DisablePelvis = Output.Curve.Get(PelvisSettings.DisablePelvisCurveName);
	PelvisTransformCS.BlendWith(PelvisData.InputPose.FKTransformCS, PelvisData.DisablePelvis);
	
	check(!PelvisTransformCS.ContainsNaN());
	OutBoneTransforms.Add(FBoneTransform(PelvisData.Bones.FkBoneIndex, PelvisTransformCS));

	if (InterpolationSettings.bSmoothRootBone)
	{
		// Smooth out the root by the same factor as the hips.
		FTransform RootBoneTransform = Output.Pose.GetComponentSpaceTransform(FCompactPoseBoneIndex(0));
		const FVector TranslationDelta = PelvisTransformCS.GetTranslation() - PelvisData.InputPose.FKTransformCS.GetTranslation();

		RootBoneTransform.AddToTranslation(TranslationDelta);
		check(!RootBoneTransform.ContainsNaN());
		OutBoneTransforms.Add(FBoneTransform(FCompactPoseBoneIndex(0), RootBoneTransform));
	}

	for (int32 FootIndex = 0; FootIndex < LegsData.Num(); ++FootIndex)
	{
		UE::Anim::FootPlacement::FLegRuntimeData& LegData = LegsData[FootIndex];
		const FFootPlacemenLegDefinition& LegDef = LegDefinitions[FootIndex];

		const UE::Anim::FootPlacement::FPlantResult PlantResult =
			FinalizeFootAlignment(FootPlacementContext, LegData, LegDef, PelvisTransformCS);

	
		
		OutBoneTransforms.Add(PlantResult.FootTranformCS);
		//OutBoneTransforms.Add(PlantResult.BallTransformCS);
		//OutBoneTransforms.Add(PlantResult.HipTransformCS);

#if ENABLE_FOOTPLACEMENT_DEBUG
		if (FVisualLogger::IsRecording())
		{
			DrawVLog(FootPlacementContext, LegData, PlantResult);
		}
#endif

#if ENABLE_FOOTPLACEMENT_DEBUG
		if (CVarAnimNodeFootPlacementDebug.GetValueOnAnyThread())
		{
			DrawDebug(FootPlacementContext, LegData, PlantResult);

			// Grab positions to debug draw history
			DebugData.OutputFootLocationsWS[FootIndex] =
				FootPlacementContext.OwningComponentToWorld.TransformPosition(PlantResult.FootTranformCS.Transform.GetLocation());
			DebugData.InputFootLocationsWS[FootIndex] = 
				FootPlacementContext.OwningComponentToWorld.TransformPosition(LegData.InputPose.FootTransformCS.GetLocation());;
		}
#endif
	}

	OutBoneTransforms.Sort(FCompareBoneTransformIndex());

	CachedDeltaTime = 0.0f;

#if ENABLE_FOOTPLACEMENT_DEBUG
	{
		FAnimInstanceProxy* AnimInstanceProxy = Output.AnimInstanceProxy;
		const FTransform ComponentTransform =
			AnimInstanceProxy->GetSkelMeshComponent()->GetComponentTransform();

		const FVector InputPelvisLocationWS = ComponentTransform.TransformPosition(PelvisData.InputPose.FKTransformCS.GetLocation());
		const FVector OutputPelvisLocationWS = ComponentTransform.TransformPosition(PelvisTransformCS.GetLocation());

		DebugData.InputPelvisLocationWS = InputPelvisLocationWS;
		DebugData.OutputPelvisLocationWS = OutputPelvisLocationWS;

		if (CVarAnimNodeFootPlacementDebug.GetValueOnAnyThread())
		{
			const int32 DrawIndex = CVarAnimNodeFootPlacementDebugDrawHistory.GetValueOnAnyThread();
			if ((DrawIndex != 0) && !bIsFirstUpdate)
			{
				if (DrawIndex == -1)
				{
					AnimInstanceProxy->AnimDrawDebugLine(LastDebugData.OutputPelvisLocationWS, DebugData.OutputPelvisLocationWS, FColor::Magenta, true, -1.0f, 0.5f);
					AnimInstanceProxy->AnimDrawDebugLine(LastDebugData.InputPelvisLocationWS, DebugData.InputPelvisLocationWS, FColor::Blue, true, -1.0f, 0.5f);
				}
				if (DrawIndex > 0 && DebugData.OutputFootLocationsWS.IsValidIndex(DrawIndex - 1))
				{
					const int32 FootIndex = DrawIndex - 1;
					AnimInstanceProxy->AnimDrawDebugLine(DebugData.OutputFootLocationsWS[FootIndex], LastDebugData.OutputFootLocationsWS[FootIndex], FColor::Magenta, true, -1.0f, 0.5f);
					AnimInstanceProxy->AnimDrawDebugLine(DebugData.InputFootLocationsWS[FootIndex], LastDebugData.InputFootLocationsWS[FootIndex], FColor::Blue, true, -1.0f, 0.5f);
				}
			}

			const FTransform PelvisTransformWS = PelvisTransformCS * ComponentTransform;
			const FTransform BasePelvisTransformWS = PelvisData.InputPose.FKTransformCS * ComponentTransform;
			const FTransform PelvisTargetTransformWS = PelvisTargetTransformCS * ComponentTransform;

			AnimInstanceProxy->AnimDrawDebugPoint(
				PelvisTransformWS.GetLocation(), 20.0f, FColor::Green, false, -1.0f, SDPG_Foreground);

			AnimInstanceProxy->AnimDrawDebugPoint(
				BasePelvisTransformWS.GetLocation(), 20.0f, FColor::Blue, false, -1.0f, SDPG_Foreground);

			// Draw pelvis interpolation target
			AnimInstanceProxy->AnimDrawDebugPoint(
				PelvisTargetTransformWS.GetLocation(), 10.0f, FColor::Purple, false, -1.0f, SDPG_Foreground);

			const FVector IKFootRootLocationWS = ComponentTransform.TransformPosition(PelvisData.InputPose.IKRootTransformCS.GetLocation());
			AnimInstanceProxy->AnimDrawDebugCircle(
				IKFootRootLocationWS, 100.0f, 24, FColor::Cyan,
				CharacterData.SmoothCapsuleGroundNormalWS, false, -1.0f, SDPG_Foreground, 0.5f);
		}
		
#if ENABLE_FOOTPLACEMENT_DEBUG
		if (FVisualLogger::IsRecording())
		{
			// pelvis debugging
			const FTransform PelvisTransformWS = PelvisTransformCS * ComponentTransform;
			const FTransform BasePelvisTransformWS = PelvisData.InputPose.FKTransformCS * ComponentTransform;
			const FTransform PelvisTargetTransformWS = PelvisTargetTransformCS * ComponentTransform;

			UObject* AnimInstance = AnimInstanceProxy->GetAnimInstanceObject();
			UE_VLOG_SPHERE(AnimInstance, "FootPlacement", Display, PelvisTransformWS.GetTranslation(), 0, FColor::Green, TEXT(""));
			UE_VLOG_SPHERE(AnimInstance, "FootPlacement", Display, BasePelvisTransformWS.GetTranslation(), 0, FColor::Blue, TEXT(""));
			UE_VLOG_SPHERE(AnimInstance, "FootPlacement", Display, PelvisTargetTransformWS.GetTranslation(), 0, FColor::Purple, TEXT(""));

			const FVector IKFootRootLocationWS = ComponentTransform.TransformPosition(PelvisData.InputPose.IKRootTransformCS.GetLocation());
			UE_VLOG_CIRCLE_THICK(AnimInstance, "FootPlacement", Display, IKFootRootLocationWS, CharacterData.SmoothCapsuleGroundNormalWS, 50, FColor::Cyan, 1, TEXT(""));
		}
#endif
	}
#endif

	LastComponentLocation = FootPlacementContext.OwningComponentToWorld.GetLocation();

	bIsFirstUpdate = false;
}

bool FAnimNode_FootPlacement::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones)
{
#if ENABLE_FOOTPLACEMENT_DEBUG
	if (!CVarAnimNodeFootPlacementEnable.GetValueOnAnyThread())
	{
		return false;
	}
#endif

	for (const UE::Anim::FootPlacement::FLegRuntimeData& LegData : LegsData)
	{
		if (!LegData.Bones.HipIndex.IsValid() ||
			!LegData.Bones.FKIndex.IsValid() ||
			!LegData.Bones.IKIndex.IsValid() ||
			!LegData.Bones.BallIndex.IsValid())
		{
			return false;
		}
	}

	if (!PelvisData.Bones.IkBoneIndex.IsValid() ||
		!PelvisData.Bones.FkBoneIndex.IsValid())
	{
		return false;
	}

	return true;
}

void FAnimNode_FootPlacement::InitializeBoneReferences(const FBoneContainer& RequiredBones)
{
	for (int32 FootIndex = 0; FootIndex < LegsData.Num(); ++FootIndex)
	{
		FFootPlacemenLegDefinition& LegDef = LegDefinitions[FootIndex];
		UE::Anim::FootPlacement::FLegRuntimeData& LegData = LegsData[FootIndex];
		LegDef.IKFootBone.Initialize(RequiredBones);
		LegDef.FKFootBone.Initialize(RequiredBones);
		LegDef.BallBone.Initialize(RequiredBones);

		LegData.Bones.IKIndex = LegDef.IKFootBone.GetCompactPoseIndex(RequiredBones);
		LegData.Bones.FKIndex = LegDef.FKFootBone.GetCompactPoseIndex(RequiredBones);
		LegData.Bones.BallIndex = LegDef.BallBone.GetCompactPoseIndex(RequiredBones);

		if ((!LegData.Bones.IKIndex.IsValid()) ||
			(!LegData.Bones.FKIndex.IsValid()) ||
			(!LegData.Bones.BallIndex.IsValid()))
		{
			break;
		}

		UE::Anim::FootPlacement::FindChainLengthRootBoneIndex(
			LegData.Bones.FKIndex, FMath::Max(LegDef.NumBonesInLimb, 1), RequiredBones,
			LegData.Bones.HipIndex, LegData.Bones.LimbLength);

		const FTransform BallTransformLS = RequiredBones.GetRefPoseTransform(LegData.Bones.BallIndex);
		LegData.Bones.FootLength = BallTransformLS.GetLocation().Size();

#if ENABLE_FOOTPLACEMENT_DEBUG
		// TODO: This wont work for animations authored for different slopes or stairs. Figure this out later
		const FVector RefPoseGroundNormalCS = FVector::UpVector;
		const FTransform BallRefTransformCS = FAnimationRuntime::GetComponentSpaceRefPose(
			LegData.Bones.BallIndex,
			RequiredBones);
		const FVector BallAlignmentDeltaCS = -BallRefTransformCS.GetLocation();
		const FVector BallAlignmenfOffsetCS = (BallAlignmentDeltaCS | RefPoseGroundNormalCS) * RefPoseGroundNormalCS;
		LegData.InputPose.BallToGround = FTransform(BallRefTransformCS.GetRotation().UnrotateVector(BallAlignmenfOffsetCS));

		const FTransform FKFootTransformCS = FAnimationRuntime::GetComponentSpaceRefPose(
			LegData.Bones.FKIndex,
			RequiredBones);
		const FVector FootAlignmentDeltaCS = -FKFootTransformCS.GetLocation();
		const FVector FootAlignmentOffsetCS = (FootAlignmentDeltaCS | RefPoseGroundNormalCS) * RefPoseGroundNormalCS;
		LegData.InputPose.FootToGround = FTransform(FKFootTransformCS.GetRotation().UnrotateVector(FootAlignmentOffsetCS));
#endif 

		LegData.SpeedCurveName = LegDef.SpeedCurveName;
		LegData.DisableLockCurveName = LegDef.DisableLockCurveName;
		LegData.DisableLegCurveName = LegDef.DisableLegCurveName;
	}

	PelvisBone.Initialize(RequiredBones);
	IKFootRootBone.Initialize(RequiredBones);

	PelvisData.Bones.FkBoneIndex = PelvisBone.GetCompactPoseIndex(RequiredBones);
	PelvisData.Bones.IkBoneIndex = IKFootRootBone.GetCompactPoseIndex(RequiredBones);
}

void FAnimNode_FootPlacement::GatherPelvisDataFromInputs(const UE::Anim::FootPlacement::FEvaluationContext& Context)
{
	PelvisData.InputPose.FKTransformCS =
		Context.CSPContext.Pose.GetComponentSpaceTransform(PelvisData.Bones.FkBoneIndex);
	PelvisData.InputPose.IKRootTransformCS =
		Context.CSPContext.Pose.GetComponentSpaceTransform(PelvisData.Bones.IkBoneIndex);

	PelvisData.InputPose.RootTransformCS =
		Context.CSPContext.Pose.GetComponentSpaceTransform(FCompactPoseBoneIndex(0));

	// TODO: All of these can be calculated on initialize, but in case there's value in changing these dynamically,
	// will keep this for now. If needed, change to lazy update.
	PelvisData.MaxOffsetSqrd = PelvisSettings.MaxOffset * PelvisSettings.MaxOffset;
}

void FAnimNode_FootPlacement::GatherLegDataFromInputs(
	const UE::Anim::FootPlacement::FEvaluationContext& Context,
	UE::Anim::FootPlacement::FLegRuntimeData& LegData,
	const FFootPlacemenLegDefinition& LegDef)
{
	FVector LastBallLocation = LegData.InputPose.BallTransformCS.GetLocation();

	LegData.InputPose.FootFKTransformCS =
		Context.CSPContext.Pose.GetComponentSpaceTransform(LegData.Bones.FKIndex);
	const FTransform BallTransformCS =
		Context.CSPContext.Pose.GetComponentSpaceTransform(LegData.Bones.BallIndex);

	LegData.InputPose.FootTransformCS =
		Context.CSPContext.Pose.GetComponentSpaceTransform(LegData.Bones.IKIndex);
	LegData.InputPose.HipTransformCS =
		Context.CSPContext.Pose.GetComponentSpaceTransform(LegData.Bones.HipIndex);
	
	LegData.InputPose.BallToFoot =
		LegData.InputPose.FootFKTransformCS.GetRelativeTransform(BallTransformCS);
	LegData.InputPose.FootToBall =
		BallTransformCS.GetRelativeTransform(LegData.InputPose.FootFKTransformCS);

	// Can't use ball transform as-is as the foot's IK bone may not be at the FK bone.
	// Assume the ball is at the same relative position
	LegData.InputPose.BallTransformCS =
		LegData.InputPose.FootToBall * LegData.InputPose.FootTransformCS;

	if (bIsFirstUpdate)
	{
		const FTransform& RootToComponent = GetRootToComponent();

		LegData.AlignedFootTransformRS = LegData.InputPose.FootTransformCS.GetRelativeTransform(RootToComponent);
		LegData.AlignedFootTransformWS = LegData.InputPose.FootTransformCS * Context.OwningComponentToWorld;
		LegData.UnalignedFootTransformRS = LegData.AlignedFootTransformRS;
		LegData.UnalignedFootTransformWS = LegData.AlignedFootTransformWS;

		const FVector IKFootRootLocationWS =
			Context.OwningComponentToWorld.TransformPosition(PelvisData.InputPose.IKRootTransformCS.GetLocation());

		const FVector IKFootRootLocationRS = RootToComponent.InverseTransformPosition(PelvisData.InputPose.IKRootTransformCS.GetLocation());
		LegData.Plant.PlantPlaneRS = FPlane(IKFootRootLocationRS, -Context.ApproachDirCS);

		LegData.Plant.PlantType = UE::Anim::FootPlacement::EPlantType::Unplanted;
		LegData.Plant.LastPlantType = UE::Anim::FootPlacement::EPlantType::Unplanted;
		LastBallLocation = LegData.InputPose.BallTransformCS.GetLocation();
	}

	if (PlantSpeedMode == EWarpingEvaluationMode::Graph)
	{
		FVector BallTranslationDelta = LegData.InputPose.BallTransformCS.GetLocation() - LastBallLocation;

		// Apply root motion delta to the ball's translation delta in root space
		const FQuat RootRotation = Context.CSPContext.Pose.GetComponentSpaceTransform(FCompactPoseBoneIndex(0)).GetRotation();
		const FVector CorrectedRootMotionTranslationDelta = RootRotation.RotateVector(Context.RootMotionTransformDelta.GetTranslation());
		BallTranslationDelta += CorrectedRootMotionTranslationDelta;

		const float BallDeltaDistance = BallTranslationDelta.Size();
		LegData.InputPose.Speed = BallDeltaDistance / Context.UpdateDeltaTime;
	}
	else
	{
		bool bValidSpeedCurve;
		// If the curve is not found in the stream, assume we're unplanted.
		const float DefaultSpeedCurveValue = PlantSettings.SpeedThreshold;
		LegData.InputPose.Speed =
			Context.CSPContext.Curve.Get(LegData.SpeedCurveName, bValidSpeedCurve, DefaultSpeedCurveValue);
	}

	LegData.InputPose.DisableLeg = Context.CSPContext.Curve.Get(LegData.DisableLegCurveName);

	// Grab the lock curve's alpha. If the curve isn't set, then LockAlpha is full weight.
	LegData.InputPose.LockAlpha = 1.0f - Context.CSPContext.Curve.Get(LegData.DisableLockCurveName);

	LegData.InputPose.DistanceToPlant = CalcTargetPlantPlaneDistance(Context, LegData.InputPose);
	const float FKAlignmentAlpha = GetAlignmentAlpha(Context, LegData.InputPose);
	LegData.InputPose.AlignmentAlpha = FKAlignmentAlpha;
}

void FAnimNode_FootPlacement::CalculateFootMidpoint(const UE::Anim::FootPlacement::FEvaluationContext& Context, TConstArrayView<UE::Anim::FootPlacement::FLegRuntimeData> InLegsData, FVector& OutMidpoint) const
{
	int32 NumLegs = LegsData.Num();
	OutMidpoint = FVector::ZeroVector;
	for (const UE::Anim::FootPlacement::FLegRuntimeData& LegData : InLegsData)
	{
		OutMidpoint += LegData.InputPose.FootTransformCS.GetLocation() / NumLegs;
	}
}

void FAnimNode_FootPlacement::ProcessCharacterState(const UE::Anim::FootPlacement::FEvaluationContext& Context)
{
	const FVector LastComponentLocationWS = bIsFirstUpdate
		? Context.OwningComponentToWorld.GetLocation()
		: CharacterData.ComponentTransformWS.GetLocation();

	if (bIsFirstUpdate)
	{
		CharacterData.SmoothCapsuleGroundNormalWS = -Context.ApproachDirWS;
		CharacterData.SmoothCapsuleGroundNormalSpringState.Reset();
	}

	CharacterData.ComponentTransformWS = Context.OwningComponentToWorld;
	const FVector ComponentLocationWS = CharacterData.ComponentTransformWS.GetLocation();

	const bool bWasOnGround = CharacterData.bIsOnGround;
	CharacterData.bIsOnGround = !Context.MovementComponent ?
		true :
		((Context.MovementComponent->MovementMode == MOVE_Walking) ||
			(Context.MovementComponent->MovementMode == MOVE_NavWalking)) &&
		Context.MovementComponent->CurrentFloor.bBlockingHit;

	CharacterData.ComponentMoveDeltaWS = FVector::ZeroVector;
	bool bOnGround =  !PelvisSettings.bDisablePelvisOffsetInAir || (CharacterData.bIsOnGround && bWasOnGround);
	if (bOnGround && PelvisSettings.ActorMovementCompensationMode != EActorMovementCompensationMode::ComponentSpace)
	{
		FVector OwningComponentAdjustedLastLocationWS;
		if (PelvisSettings.ActorMovementCompensationMode == EActorMovementCompensationMode::SuddenMotionOnly)
		{
			FQuat SlopeDelta = FQuat::FindBetweenNormals(CharacterData.SmoothCapsuleGroundNormalWS, Context.GetMovementComponentFloorNormal());
			SlopeDelta = UKismetMathLibrary::QuaternionSpringInterp(
																FQuat::Identity, 
																SlopeDelta, 
																CharacterData.SmoothCapsuleGroundNormalSpringState, 
																InterpolationSettings.FloorAngularStiffness, 
																1.0f, 
																Context.UpdateDeltaTime, 
																1.0f, 
																0.0f);
			CharacterData.SmoothCapsuleGroundNormalWS = SlopeDelta.RotateVector(CharacterData.SmoothCapsuleGroundNormalWS);

			// Compensate for sudden capsule moves
			const FVector CapsuleFloorNormalWS = CharacterData.SmoothCapsuleGroundNormalWS;
			OwningComponentAdjustedLastLocationWS =
				(FMath::Abs(Context.ApproachDirWS | CapsuleFloorNormalWS) > DELTA) ?
				UE::Anim::FootPlacement::PointDirectionPlaneIntersection(
					ComponentLocationWS,
					Context.ApproachDirWS,
					FPlane(LastComponentLocationWS, CapsuleFloorNormalWS)) :
				ComponentLocationWS;
		}
		else //if (PelvisSettings.ActorMovementCompensationMode == EActorMovementCompensationMode::WorldSpace)
		{
			// Compensate for all moves
			OwningComponentAdjustedLastLocationWS = LastComponentLocationWS;
		}

		// Only compensate vertical motion
		const FVector CapsuleMoveOffsetWS =
			(ComponentLocationWS - OwningComponentAdjustedLastLocationWS - BaseTranslationDelta) * -Context.ApproachDirWS;

		CharacterData.ComponentMoveDeltaWS -= CapsuleMoveOffsetWS;
		if (!CapsuleMoveOffsetWS.IsNearlyZero(KINDA_SMALL_NUMBER))
		{
			const FVector CapsuleMoveOffsetCS =
				Context.OwningComponentToWorld.InverseTransformVectorNoScale(CapsuleMoveOffsetWS);

			// Offseting our interpolator lets it smoothly solve sudden capsule deltas, instead of following it and pop
			PelvisData.Interpolation.PelvisTranslationOffset -= CapsuleMoveOffsetCS;

			for (UE::Anim::FootPlacement::FLegRuntimeData& LegData : LegsData)
			{
				// Also offset our foot plant plane interpolators by this same delta.
				//@TODO: this should be in root space too, but vertical motion is the same regardless of space for now.
				LegData.Plant.PlantPlaneRS = LegData.Plant.PlantPlaneRS.TranslateBy(-CapsuleMoveOffsetCS);
			}
		}
	}

	{
		CharacterData.CharacterVelocityWS = Context.GetCharacterVelocity();

		// Also grab the component's move delta, instead of the movement component's velocity, since it doesn't account for uphill/downhill velocity
		const FVector CapsuleMoveOffsetWS =
			(ComponentLocationWS - LastComponentLocationWS);
		const FVector CapsuleMoveOffsetCS =
			Context.OwningComponentToWorld.InverseTransformVectorNoScale(CapsuleMoveOffsetWS);
		CharacterData.ComponentMoveDeltaWS += CapsuleMoveOffsetWS;
	}
}

void FAnimNode_FootPlacement::ProcessFootAlignment(
	const UE::Anim::FootPlacement::FEvaluationContext& Context,
	UE::Anim::FootPlacement::FLegRuntimeData& LegData)
{
	UE::Anim::FootPlacement::FLegRuntimeData::FInputPoseData& InputPose = LegData.InputPose;
	UE::Anim::FootPlacement::FLegRuntimeData::FInterpolationData& Interpolation = LegData.Interpolation;
	UE::Anim::FootPlacement::FLegRuntimeData::FBoneData& Bones = LegData.Bones;
	UE::Anim::FootPlacement::FLegRuntimeData::FPlantData& Plant = LegData.Plant;
	
	const FTransform& RootToComponent = GetRootToComponent();
	if (PlantSettings.bReconstructWorldPlantFromVelocity)
	{
		// Last frame's plant in root space minus our move delta from character velocity.
		LegData.AlignedFootTransformWS = LegData.AlignedFootTransformRS * RootToComponent * Context.OwningComponentToWorld;
		LegData.AlignedFootTransformWS.AddToTranslation(-CharacterData.CharacterVelocityWS * Context.UpdateDeltaTime);

		LegData.UnalignedFootTransformWS = LegData.UnalignedFootTransformRS * RootToComponent * Context.OwningComponentToWorld;
		LegData.UnalignedFootTransformWS.AddToTranslation(-CharacterData.CharacterVelocityWS * Context.UpdateDeltaTime);
	}
	else
	{
		LegData.AlignedFootTransformWS.AddToTranslation(-BaseTranslationDelta);
		LegData.UnalignedFootTransformWS.AddToTranslation(-BaseTranslationDelta);
	}

	const FTransform InputPoseFootTransformWS = InputPose.FootTransformCS * Context.OwningComponentToWorld;
	const FTransform LastAlignedFootTransformWS = LegData.AlignedFootTransformWS;
	const FTransform LastUnalignedFootTransformWS = LegData.UnalignedFootTransformWS;
	const FTransform InputPoseFootTransformRS = InputPose.FootTransformCS.GetRelativeTransform(RootToComponent);

	Plant.LastPlantType = Plant.PlantType;
	DeterminePlantType(
		Context,
		InputPoseFootTransformWS,
		LastAlignedFootTransformWS,
		Plant,
		InputPose);

	const bool bIsPlanted = Plant.PlantType != UE::Anim::FootPlacement::EPlantType::Unplanted;
	const bool bWasPlanted = Plant.LastPlantType != UE::Anim::FootPlacement::EPlantType::Unplanted;

	if (bIsPlanted)
	{
		FTransform CurrentPlantedTransformWS;

		switch (PlantSettings.LockType)
		{
		case EFootPlacementLockType::Unlocked:
			break;
		case EFootPlacementLockType::PivotAroundBall:
		{
			// Figure out the correct foot transform that keeps the ball in place
			CurrentPlantedTransformWS = GetFootPivotAroundBallWS(Context, InputPose, LastUnalignedFootTransformWS);;
		}
			break;
		case EFootPlacementLockType::PivotAroundAnkle:
		{
			// Use the location only
			CurrentPlantedTransformWS = InputPoseFootTransformWS;
			CurrentPlantedTransformWS.SetLocation(LastUnalignedFootTransformWS.GetLocation());
		}
			break;
		case EFootPlacementLockType::LockRotation:
		{
			// We use the unaligned foot instead of the aligned one
			// Because we will adjust roll and twist dynamically
			CurrentPlantedTransformWS = LastUnalignedFootTransformWS;
		}
			break;
		default: check(false); break; //not implemented
		}

		const FTransform PlantedFootTransformCS =
			CurrentPlantedTransformWS * Context.OwningComponentToWorld.Inverse();
		FTransform PlantedFootTransformRS = PlantedFootTransformCS.GetRelativeTransform(RootToComponent);

		// The locked transform is aligned to the ground. Conserve the input pose's ground alignment
		const FVector AlignedBoneLocationRS = PlantedFootTransformRS.GetLocation();
		const FPlane InputPosePlantPlane = FPlane(InputPoseFootTransformRS.GetLocation(), Context.ApproachDirWS);
		const FVector UnalignedBoneLocationRS = FVector::PointPlaneProject(AlignedBoneLocationRS, InputPosePlantPlane);

		PlantedFootTransformRS.SetLocation(UnalignedBoneLocationRS);

		// Get the offset relative to the initial foot transform
		// Reset interpolation 
		Interpolation.UnalignedFootOffset =
			InputPoseFootTransformRS.GetRelativeTransformReverse(PlantedFootTransformRS);
		Interpolation.PlantOffsetTranslationSpringState.Reset();
		Interpolation.PlantOffsetRotationSpringState.Reset();

		// If we planted, we're fully unaligned
		Plant.TimeSinceFullyUnaligned = 0.0f;
	}
	else
	{
		// No plant, so we interpolate the offset out
		Interpolation.UnalignedFootOffset =
			UpdatePlantOffsetInterpolation(Context, Interpolation);

		// If we're unplanted, we know we're fully unaligned the first time we hit zero alignment alpha.
		if (Plant.TimeSinceFullyUnaligned > 0.0f || FMath::IsNearlyZero(InputPose.AlignmentAlpha))
		{
			Plant.TimeSinceFullyUnaligned += Context.UpdateDeltaTime;
		}
	}

	// If replant radius is the same as unplant radius, clamp the location and slide
	if (PlantSettings.ReplantRadiusRatio >= 1.0f)
	{
		const FVector ClampedTransltionOffset = Interpolation.UnalignedFootOffset.GetLocation().GetClampedToMaxSize(PlantSettings.UnplantRadius);
		Interpolation.UnalignedFootOffset.SetLocation(ClampedTransltionOffset);
	}

	// If replant angle is the same as unplant angle, clamp the angle and slide
	if (PlantSettings.ReplantAngleRatio >= 1.0f)
	{
		FQuat ClampedRotationOffset = Interpolation.UnalignedFootOffset.GetRotation();
		ClampedRotationOffset.Normalize();
		ClampedRotationOffset = ClampedRotationOffset.W < 0.0 ? -ClampedRotationOffset : ClampedRotationOffset;

		FVector OffsetAxis;
		float OffsetAngle;
		ClampedRotationOffset.ToAxisAndAngle(OffsetAxis, OffsetAngle);

		const float MaxAngle = FMath::DegreesToRadians(PlantSettings.UnplantAngle);
		if (FMath::Abs(OffsetAngle) > MaxAngle)
		{
			ClampedRotationOffset = FQuat(OffsetAxis, MaxAngle);
		}
		Interpolation.UnalignedFootOffset.SetRotation(ClampedRotationOffset);
	}

	FTransform FootUnalignedTransformRS = InputPose.FootTransformCS.GetRelativeTransform(RootToComponent) * Interpolation.UnalignedFootOffset;
	//@TODO: Change this logic to be in root space.
	if (PlantSettings.SeparatingDistance > 0.0f)
	{
		// Prevent the feet from crossing by enforcing a set distance from a plane at the midpoint between all feet
		FVector FootUnalignedLocationRS = FootUnalignedTransformRS.GetLocation();
		const FVector MidPointToFoot = (InputPose.FootTransformCS.GetLocation() - PelvisData.InputPose.FootMidpointCS);
		const FVector PlaneNormal = MidPointToFoot.GetSafeNormal2D();
		const FVector PlaneCenter = PelvisData.InputPose.FootMidpointCS + PlaneNormal * PlantSettings.SeparatingDistance;
		const FPlane SeparatingPlane = FPlane(PlaneCenter, PlaneNormal);
		
		const TOptional<float> DistanceToSeparatingPlane =
			UE::Anim::FootPlacement::GetDistanceToPlaneAlongDirection(FootUnalignedLocationRS, SeparatingPlane, -PlaneNormal);

		if (Plant.PlantType == UE::Anim::FootPlacement::EPlantType::Unplanted)
		{
			FVector SeparatingPlaneOffset(FVector::ZeroVector);
			if (DistanceToSeparatingPlane.Get(0.0f) < 0.0f)
			{
				SeparatingPlaneOffset = -PlaneNormal * DistanceToSeparatingPlane.GetValue() ;
			}

			if (InterpolationSettings.bEnableSeparationInterpolation)
			{
				Interpolation.SeparatingPlaneOffset = UKismetMathLibrary::VectorSpringInterp(
					Interpolation.SeparatingPlaneOffset, SeparatingPlaneOffset, Interpolation.SeparatingPlaneOffsetSpringState,
					InterpolationSettings.FloorLinearStiffness,
					InterpolationSettings.FloorLinearDamping,
					Context.UpdateDeltaTime, 1.0f, 0.0f);
			}
			else
			{
				Interpolation.SeparatingPlaneOffset = SeparatingPlaneOffset;
			}
			
			FootUnalignedLocationRS  += Interpolation.SeparatingPlaneOffset;
			FootUnalignedTransformRS.SetLocation(FootUnalignedLocationRS);
		}
		else
		{
			Interpolation.SeparatingPlaneOffset = FVector::ZeroVector;
			Interpolation.SeparatingPlaneOffsetSpringState.Reset();
		}

#if ENABLE_FOOTPLACEMENT_DEBUG
		if (CVarAnimNodeFootPlacementDebug.GetValueOnAnyThread())
		{
			const FVector PlaneNormalWS = Context.OwningComponentToWorld.TransformVector(PlaneNormal);;
			const FVector PlaneCenterWS = Context.OwningComponentToWorld.TransformPosition(PlaneCenter);;
			Context.CSPContext.AnimInstanceProxy->AnimDrawDebugCircle(
				PlaneCenterWS, 25.0f, 24, FColor::Red,
				PlaneNormalWS, false, -1.0f, SDPG_Foreground, 0.5f);

			DebugData.LegsInfo[LegData.Idx].DistanceToSeparatingPlane = DistanceToSeparatingPlane.Get(0.0f);
		}
#endif
		
#if ENABLE_FOOTPLACEMENT_DEBUG
		if (FVisualLogger::IsRecording())
		{
			const FVector PlaneNormalWS = Context.OwningComponentToWorld.TransformVector(PlaneNormal);;
			const FVector PlaneCenterWS = Context.OwningComponentToWorld.TransformPosition(PlaneCenter);;
			
			UE_VLOG_CIRCLE_THICK(Context.CSPContext.AnimInstanceProxy->GetAnimInstanceObject(), "FootPlacement", Display, PlaneCenterWS, PlaneNormalWS, 25, FColor::Red, 1, TEXT(""));
		}
#endif
	}

	FTransform BlendedUnalignedTransformRS;
	{
		// Blend the component-space input pose, with the unaligned foot-locked transform.
		// Allow ground alignment to continue with this blended result.
		// When the lock alpha reaches 0, we will automatically unlock the foot.
		BlendedUnalignedTransformRS.Blend(InputPoseFootTransformRS, FootUnalignedTransformRS, InputPose.LockAlpha);
		LegData.UnalignedFootTransformRS = BlendedUnalignedTransformRS;
		LegData.UnalignedFootTransformWS = BlendedUnalignedTransformRS * RootToComponent * Context.OwningComponentToWorld;
	}

	const FTransform ComponentToWorldInv = Context.OwningComponentToWorld.Inverse();

	// find the smooth plant plane
	FPlane PlantPlaneWS = Plant.GetPlantPlaneWS(RootToComponent, Context.OwningComponentToWorld);
	UpdatePlantingPlaneInterpolation(Context, LegData.UnalignedFootTransformWS,
									LastAlignedFootTransformWS, 
									InputPose.AlignmentAlpha, 
									PlantPlaneWS, 
									InputPose, 
									Interpolation);
	const FPlane PlantPlaneCS = PlantPlaneWS.TransformBy(ComponentToWorldInv.ToMatrixWithScale());
	Plant.PlantPlaneRS = PlantPlaneCS.TransformBy(RootToComponent.Inverse().ToMatrixWithScale());

	// This will adjust UnalignedFootTransformWS to make it match the required distance to the plant plane along the 
	// approach direction, not the plane normal
	LegData.AlignedFootTransformWS = LegData.UnalignedFootTransformWS;
	AlignPlantToGround(Context, PlantPlaneWS, InputPose, LegData.AlignedFootTransformWS, Plant.TwistCorrection);

	const FTransform AlignedFootTransformCS =	LegData.AlignedFootTransformWS * ComponentToWorldInv;
	LegData.AlignedFootTransformRS = AlignedFootTransformCS.GetRelativeTransform(RootToComponent);
	LegData.AlignedFootTransformWS = AlignedFootTransformCS * Context.OwningComponentToWorld;
}

FVector FAnimNode_FootPlacement::GetApproachDirWS(const FAnimationBaseContext& Context) const
{
	const USkeletalMeshComponent* OwningComponent = Context.AnimInstanceProxy->GetSkelMeshComponent();
	return -(OwningComponent->GetComponentTransform().GetRotation().GetUpVector());
}

const FTransform& FAnimNode_FootPlacement::GetRootToComponent() const
{
	return PelvisData.InputPose.RootTransformCS;
}

FTransform FAnimNode_FootPlacement::SolvePelvis(const UE::Anim::FootPlacement::FEvaluationContext& Context)
{
	using namespace UE::Anim::FootPlacement;

	// Rebalance the pelvis before calculating its desired height
	FTransform RebalancedPelvisTransform  = PelvisData.InputPose.FKTransformCS;
	FVector PelvisOffsetDelta;
	if (PelvisSettings.HorizontalRebalancingWeight)
	{
		const int32 NumLegs = LegsData.Num();
		FVector OffsetAverage = FVector::ZeroVector;
		for (const FLegRuntimeData& LegData : LegsData)
		{
			const FVector LegTranslationOffset = GetRootToComponent().TransformPosition(LegData.AlignedFootTransformRS.GetLocation()) - LegData.InputPose.FootTransformCS.GetLocation();
			OffsetAverage += LegTranslationOffset / NumLegs;
		}

		// Remove the vertical component
		PelvisOffsetDelta = (OffsetAverage - Context.ApproachDirCS.Dot(OffsetAverage) * Context.ApproachDirCS) * PelvisSettings.HorizontalRebalancingWeight;
		RebalancedPelvisTransform.SetLocation(RebalancedPelvisTransform.GetLocation() + PelvisOffsetDelta);
	}

	// Taken from http://runevision.com/thesis/rune_skovbo_johansen_thesis.pdf
	// Chapter 7.4.2

	float MaxOffsetMin = BIG_NUMBER;
	float DesiredOffsetMin = BIG_NUMBER;
	float DesiredOffsetAvg = 0.0f;
	float MinOffsetMax = -BIG_NUMBER;

	const int32 FootNum = LegsData.Num();
	for (const FLegRuntimeData& LegData : LegsData)
	{
		FPelvisOffsetRangeForLimb PelvisOffsetRangeCS;
		FindPelvisOffsetRangeForLimb(
			Context,
			LegData,
			GetRootToComponent().TransformPosition(LegData.AlignedFootTransformRS.GetLocation()),
			RebalancedPelvisTransform,
			PelvisOffsetRangeCS);

		const float DesiredOffset = PelvisOffsetRangeCS.DesiredExtension;
		const float MaxOffset = PelvisOffsetRangeCS.MaxExtension;
		const float MinOffset = PelvisOffsetRangeCS.MinExtension;

		DesiredOffsetAvg += DesiredOffset / FootNum;
		DesiredOffsetMin = FMath::Min(DesiredOffsetMin, DesiredOffset);
		MaxOffsetMin = FMath::Min(MaxOffsetMin, MaxOffset);
		MinOffsetMax = FMath::Max(MinOffsetMax, MinOffset);
	}
	const float MinToAvg = DesiredOffsetAvg - DesiredOffsetMin;
	const float MinToMax = MaxOffsetMin - DesiredOffsetMin;

	DesiredOffsetMin -= 0.05f;

	// In cases like crouching, it favors over-compressing to preserve the pose of the other leg
	// Consider working in over-compression into the formula.
	const float Divisor = MinToAvg + MinToMax;
	float PelvisOffsetZ = FMath::IsNearlyZero(Divisor) ?
		DesiredOffsetMin :
		DesiredOffsetMin + ((MinToAvg * MinToMax) / Divisor);

	// Adjust the hips to prevent over-compression
	PelvisOffsetZ = FMath::Clamp(PelvisOffsetZ, MinOffsetMax, MaxOffsetMin);
	PelvisOffsetDelta += -PelvisOffsetZ * Context.ApproachDirCS;

	FTransform PelvisTransformCS = PelvisData.InputPose.FKTransformCS;
	PelvisTransformCS.AddToTranslation(PelvisOffsetDelta);

	return PelvisTransformCS;
}

FTransform FAnimNode_FootPlacement::UpdatePelvisInterpolationRootSpace(
	const UE::Anim::FootPlacement::FEvaluationContext& Context,
	const FTransform& TargetPelvisTransformRS)
{
	const FTransform& RootTransformCS = GetRootToComponent();
	const FVector PelvisLocationRS = RootTransformCS.InverseTransformPosition(PelvisData.InputPose.FKTransformCS.GetLocation());

	FTransform OutPelvisTransform = TargetPelvisTransformRS;
	// Calculate the offset from input pose and interpolate
	FVector DesiredPelvisOffset =
		TargetPelvisTransformRS.GetLocation() - PelvisLocationRS;

	// Clamp by MaxOffset
	// Clamping the target before interpolation means we may exceed this purely do to interpolation.
	// If we clamp after, you'll get no smoothing once the limit is reached.
	const float MaxOffsetSqrd = PelvisData.MaxOffsetSqrd;
	const float MaxOffset = PelvisSettings.MaxOffset;
	if (DesiredPelvisOffset.SizeSquared() > MaxOffsetSqrd)
	{
		DesiredPelvisOffset = DesiredPelvisOffset.GetClampedToMaxSize(MaxOffset);
	}

	// Spring interpolation may cause hyperextension/compression so we solve that in FinalizeFootAlignment
	const FVector NewTranslationOffset = UKismetMathLibrary::VectorSpringInterp(
		PelvisData.Interpolation.PelvisTranslationOffset, DesiredPelvisOffset, PelvisData.Interpolation.PelvisTranslationSpringState,
		PelvisSettings.LinearStiffness,
		PelvisSettings.LinearDamping,
		Context.UpdateDeltaTime, 1.0f, 0.0f);
	PelvisData.Interpolation.PelvisTranslationOffset = NewTranslationOffset;

	OutPelvisTransform.SetLocation(
		PelvisLocationRS + PelvisData.Interpolation.PelvisTranslationOffset);

	return OutPelvisTransform;
}
