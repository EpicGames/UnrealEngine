// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_OffsetRootBone.h"

#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimNodeFunctionRef.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/World.h"
#include "HAL/IConsoleManager.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_OffsetRootBone)

DECLARE_CYCLE_STAT(TEXT("OffsetRootBone Eval"), STAT_OffsetRootBone_Eval, STATGROUP_Anim);

#if ENABLE_ANIM_DEBUG
TAutoConsoleVariable<int32> CVarAnimNodeOffsetRootBoneDebug(TEXT("a.AnimNode.OffsetRootBone.Debug"), 0, TEXT("Turn on visualization debugging for Offset Root Bone"));
TAutoConsoleVariable<int32> CVarAnimNodeOffsetRootBoneEnable(TEXT("a.AnimNode.OffsetRootBone.Enable"), 1, TEXT("Toggle Offset Root Bone"));
TAutoConsoleVariable<int32> CVarAnimNodeOffsetRootBoneModifyBone(TEXT("a.AnimNode.OffsetRootBone.ModifyBone"), 1, TEXT("Toggle whether the transform is applied to the bone"));
#endif

IMPLEMENT_ANIMGRAPH_MESSAGE(UE::AnimationWarping::FRootOffsetProvider);

namespace UE::Anim::OffsetRootBone
{
	// Taken from https://theorangeduck.com/page/spring-roll-call#implicitspringdamper
	static float DamperImplicit(float Halflife, float DeltaTime, float Epsilon = 1e-8f)
	{
		// Halflife values very close to 0 approach infinity, and result in big motion spikes when Halflife < DeltaTime
		// This is a hack, and adds some degree of frame-rate dependency, but it holds up even at lower frame-rates.
		Halflife = FMath::Max(Halflife, DeltaTime);
		return FMath::Clamp((1.0f - FMath::InvExpApprox((0.69314718056f * DeltaTime) / (Halflife + Epsilon))), 0.0f, 1.0f);
	}

	static bool ShouldExtractRootMotion(const EOffsetRootBoneMode OffsetMode)
	{
		switch (OffsetMode)
		{
		case EOffsetRootBoneMode::Accumulate:
		case EOffsetRootBoneMode::Interpolate:
		case EOffsetRootBoneMode::LockOffsetAndConsumeAnimation:
		case EOffsetRootBoneMode::LockOffsetIncreaseAndConsumeAnimation:
			return true;
		case EOffsetRootBoneMode::LockOffsetAndIgnoreAnimation:
		case EOffsetRootBoneMode::Release:
		default:
			return false;
		}
	}

	static bool ShouldCounterComponentDelta(const EOffsetRootBoneMode OffsetMode)
	{
		switch (OffsetMode)
		{
		case EOffsetRootBoneMode::Accumulate:
		case EOffsetRootBoneMode::Interpolate:
			return false;
		case EOffsetRootBoneMode::LockOffsetAndConsumeAnimation:
		case EOffsetRootBoneMode::LockOffsetIncreaseAndConsumeAnimation:
		case EOffsetRootBoneMode::LockOffsetAndIgnoreAnimation:
		case EOffsetRootBoneMode::Release:
		default:
			return true;
		}
	}
}

void FAnimNode_OffsetRootBone::GatherDebugData(FNodeDebugData& DebugData)
{
	Super::GatherDebugData(DebugData);

	FString DebugLine = DebugData.GetNodeName(this);
#if ENABLE_ANIM_DEBUG
	if (const UEnum* ModeEnum = StaticEnum<EOffsetRootBoneMode>())
	{
		DebugLine += FString::Printf(TEXT("\n - Translation Mode: (%s)"), *(ModeEnum->GetNameStringByIndex(static_cast<int32>(GetTranslationMode()))));
		DebugLine += FString::Printf(TEXT("\n - Rotation Mode: (%s)"), *(ModeEnum->GetNameStringByIndex(static_cast<int32>(GetRotationMode()))));
	}
	DebugLine += FString::Printf(TEXT("\n - Translation Halflife: (%.3fd)"), GetTranslationHalflife());
	DebugLine += FString::Printf(TEXT("\n - Rotation Halflife: (%.3fd)"), GetRotationHalfLife());
#endif
	DebugData.AddDebugItem(DebugLine);

	Source.GatherDebugData(DebugData);
}

void FAnimNode_OffsetRootBone::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	Super::Initialize_AnyThread(Context);
	AnimInstanceProxy = Context.AnimInstanceProxy;
	Source.Initialize(Context);
	Reset(Context);
}

void FAnimNode_OffsetRootBone::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	Super::CacheBones_AnyThread(Context);
	Source.CacheBones(Context);
}

void FAnimNode_OffsetRootBone::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	Super::Update_AnyThread(Context);
	CachedDeltaTime = Context.GetDeltaTime();

	GetEvaluateGraphExposedInputs().Execute(Context);

	// If we just became relevant and haven't been initialized yet, then reset.
	if (GetResetEveryFrame() || 
		(!bIsFirstUpdate && UpdateCounter.HasEverBeenUpdated() && !UpdateCounter.WasSynchronizedCounter(Context.AnimInstanceProxy->GetUpdateCounter())))
	{
		Reset(Context);
	}
	UpdateCounter.SynchronizeWith(Context.AnimInstanceProxy->GetUpdateCounter());
	
	UE::Anim::TScopedGraphMessage<UE::AnimationWarping::FRootOffsetProvider> ScopedMessage(Context, FTransform(SimulatedRotation, SimulatedTranslation));
	
	Source.Update(Context);
}

void FAnimNode_OffsetRootBone::Evaluate_AnyThread(FPoseContext& Output)
{
	SCOPE_CYCLE_COUNTER(STAT_OffsetRootBone_Eval);

	Super::Evaluate_AnyThread(Output);
	Source.Evaluate(Output);

#if ENABLE_ANIM_DEBUG
	if (CVarAnimNodeOffsetRootBoneEnable.GetValueOnAnyThread() == 0)
	{
		return;
	}
#endif

	bool bGraphDriven = false;
	const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get();

	if (GetEvaluationMode() == EWarpingEvaluationMode::Graph)
	{
		bGraphDriven = RootMotionProvider != nullptr;
		ensureMsgf(bGraphDriven, TEXT("Graph driven Offset Root Bone expected a valid root motion delta provider interface."));
	}

	const FCompactPoseBoneIndex TargetBoneIndex(0);
	const FTransform InputBoneTransform = Output.Pose[TargetBoneIndex];

	FTransform LastComponentTransform = ComponentTransform;
	ComponentTransform = AnimInstanceProxy->GetComponentTransform();

	// If the difference between the previous and current component transform is above the teleport threshold
	// then we adjust the simulated position to account for the teleport. Then we set the last transform to be 
	// the same as the previous so that the node does not attempt to counter-animate out the component transform 
	// this frame
	const float TeleportDistanceThreshold = Output.AnimInstanceProxy->GetSkelMeshComponent()->GetTeleportDistanceThreshold();
	if (FVector::Distance(LastComponentTransform.GetLocation(), ComponentTransform.GetLocation()) > TeleportDistanceThreshold)
	{
		if (GetResetOnTeleport())
		{
			SimulatedTranslation = ComponentTransform.GetTranslation();
			SimulatedRotation = ComponentTransform.GetRotation();
		}
		else
		{
			const FTransform OffsetTransform = FTransform(SimulatedRotation, SimulatedTranslation).GetRelativeTransformReverse(LastComponentTransform);
			const FTransform TeleportedTransform = OffsetTransform * ComponentTransform;
			SimulatedTranslation = TeleportedTransform.GetLocation();
			SimulatedRotation = TeleportedTransform.GetRotation();
		}

		LastComponentTransform = ComponentTransform;
	}

	const EOffsetRootBoneMode CurrentTranslationMode = GetTranslationMode();
	const EOffsetRootBoneMode CurrentRotationMode = GetRotationMode();

	bool bShouldConsumeTranslationOffset = UE::Anim::OffsetRootBone::ShouldExtractRootMotion(CurrentTranslationMode);
	bool bShouldConsumeRotationOffset = UE::Anim::OffsetRootBone::ShouldExtractRootMotion(CurrentRotationMode);

	FTransform RootMotionTransformDelta = FTransform::Identity;
	if (bGraphDriven)
	{
		// Graph driven mode will override translation and rotation delta
		// with the intent of the current animation sub-graph's accumulated root motion
		bGraphDriven = RootMotionProvider->ExtractRootMotion(Output.CustomAttributes, RootMotionTransformDelta);
	}
	else
	{
		// Apply the offset as is (component space) in manual mode
		RootMotionTransformDelta = FTransform(GetRotationDelta(), GetTranslationDelta());
	}

	RootMotionTransformDelta.NormalizeRotation();
	
	float MaxTranslationOffset = GetMaxTranslationError();
	
	bool bCollisionDetected = false;
	FVector CollisionPoint;
	FVector CollisionNormal;

	if (GetCollisionTestingMode() != EOffsetRootBone_CollisionTestingMode::Disabled && MaxTranslationOffset > 0)
	{
		const FCollisionShape CollisionShape = FCollisionShape::MakeSphere(GetCollisionTestShapeRadius());
		
		FVector TraceDirectionCS = FVector(0,1,0);
		if (RootMotionTransformDelta.GetTranslation().Length() > 0.1f)
		{
			TraceDirectionCS = LastNonZeroRootMotionDirection = RootMotionTransformDelta.GetTranslation().GetUnsafeNormal();
		}
		else if (LastNonZeroRootMotionDirection.SquaredLength() > UE_SMALL_NUMBER)
		{
			TraceDirectionCS = LastNonZeroRootMotionDirection;
		}
		const FVector TraceDirectionWS = SimulatedRotation.RotateVector(TraceDirectionCS);
		
		const FVector TraceStart = ComponentTransform.GetTranslation() + GetCollisionTestShapeOffset();
		const FVector TraceEnd = TraceStart + (MaxTranslationOffset * TraceDirectionWS);
    
		FCollisionQueryParams QueryParams;
		// Ignore self and all attached components
		QueryParams.AddIgnoredActor(AnimInstanceProxy->GetSkelMeshComponent()->GetOwner());
    
		const ECollisionChannel CollisionChannel = UEngineTypes::ConvertToCollisionChannel(TraceTypeQuery1);
    
		FHitResult HitResult;
		const bool bHit = AnimInstanceProxy->GetSkelMeshComponent()->GetWorld()->SweepSingleByChannel(
			HitResult, TraceStart, TraceEnd, FQuat::Identity, CollisionChannel, CollisionShape, QueryParams);
    
		if (bHit && HitResult.Distance < MaxTranslationOffset)
		{
			if (GetCollisionTestingMode() == EOffsetRootBone_CollisionTestingMode::ShrinkMaxTranslation)
			{
				MaxTranslationOffset = HitResult.Distance;
			}
			
			bCollisionDetected = true;
			CollisionPoint = HitResult.ImpactPoint;
			CollisionNormal = HitResult.ImpactNormal;
		}
	}
	

	FTransform ConsumedRootMotionDelta;

	if (bShouldConsumeTranslationOffset)
	{
		// Grab root motion translation from the root motion attribute
		ConsumedRootMotionDelta.SetTranslation(RootMotionTransformDelta.GetTranslation());
	}
	if (bShouldConsumeRotationOffset)
	{
		// Grab root motion rotation from the root motion attribute
		ConsumedRootMotionDelta.SetRotation(RootMotionTransformDelta.GetRotation());
	}

	if (UE::Anim::OffsetRootBone::ShouldCounterComponentDelta(CurrentRotationMode))
	{
		// Accumulate the rotation component delta into the simulated rotation, to keep component and offset in sync.
		const FQuat ComponentRotationDelta = LastComponentTransform.GetRotation().Inverse() * ComponentTransform.GetRotation();
		SimulatedRotation = ComponentRotationDelta * SimulatedRotation;
	}
	if (UE::Anim::OffsetRootBone::ShouldCounterComponentDelta(CurrentTranslationMode))
	{
		// Accumulate the translation component delta into the simulated translation, to keep component and offset in sync.
		const FVector ComponentTranslationDelta = ComponentTransform.GetLocation() - LastComponentTransform.GetLocation();
		SimulatedTranslation += ComponentTranslationDelta;
	}

	if (CurrentTranslationMode == EOffsetRootBoneMode::LockOffsetIncreaseAndConsumeAnimation)
	{
		FVector DeltaDir = (SimulatedTranslation - ComponentTransform.GetLocation()).GetSafeNormal();
		DeltaDir = SimulatedRotation.UnrotateVector(DeltaDir).GetSafeNormal();
		// Only allow root motion to steer us towards an position that will make the offset smaller.
		ConsumedRootMotionDelta.SetTranslation(DeltaDir * ConsumedRootMotionDelta.GetTranslation().Dot(DeltaDir));
	}

	if (CurrentRotationMode == EOffsetRootBoneMode::LockOffsetIncreaseAndConsumeAnimation)
	{
		FQuat DeltaRot = ComponentTransform.GetRotation() * SimulatedRotation.Inverse();

		FVector DeltaAxis;
		float DeltaAngle;
		DeltaRot.ToAxisAndAngle(DeltaAxis, DeltaAngle);

		float RootMotionAngle = ConsumedRootMotionDelta.GetRotation().GetTwistAngle(DeltaAxis);
		if (DeltaAngle >= 0.0f)
		{
			RootMotionAngle = FMath::Clamp(RootMotionAngle, 0.0f, DeltaAngle);
		}
		else
		{
			RootMotionAngle = FMath::Clamp(RootMotionAngle, DeltaAngle, 0.0f);
		}

		// Only allow root motion to steer us towards an orientation that will make the offset smaller.
		ConsumedRootMotionDelta.SetRotation(FQuat(DeltaAxis, RootMotionAngle));
	}

	FTransform SimulatedTransform(SimulatedRotation, SimulatedTranslation);
	// Apply the root motion delta
	SimulatedTransform = ConsumedRootMotionDelta * SimulatedTransform;

	SimulatedTranslation = SimulatedTransform.GetLocation();
	SimulatedRotation = SimulatedTransform.GetRotation();
	
	if (GetOnGround())
	{
		SimulatedTranslation = FVector::PointPlaneProject(SimulatedTranslation, ComponentTransform.GetLocation(), GetGroundNormal());
	}

	const FBoneContainer& RequiredBones = Output.Pose.GetBoneContainer();

	bool bModifyBone = true;
#if ENABLE_ANIM_DEBUG
	bModifyBone = CVarAnimNodeOffsetRootBoneModifyBone.GetValueOnAnyThread() == 1;
#endif

	if (GetTranslationMode() == EOffsetRootBoneMode::Release ||
		GetTranslationMode() == EOffsetRootBoneMode::Interpolate)
	{
		FVector TranslationOffset = ComponentTransform.GetLocation() - SimulatedTranslation;

		const float DampenAlpha = UE::Anim::OffsetRootBone::DamperImplicit(GetTranslationHalflife(), CachedDeltaTime);
		FVector TranslationOffsetDelta = FMath::Lerp(FVector::ZeroVector, TranslationOffset, DampenAlpha);

		if (GetClampToTranslationVelocity())
		{
			const float RootMotionDelta = RootMotionTransformDelta.GetLocation().Size();
			const float MaxDelta = 	GetTranslationSpeedRatio() * RootMotionDelta;

			const float AdjustmentDelta = TranslationOffsetDelta.Size();
			if (AdjustmentDelta > MaxDelta)
			{
				TranslationOffsetDelta = MaxDelta * TranslationOffsetDelta.GetSafeNormal2D();
			}
		}

		if (bCollisionDetected && GetCollisionTestingMode() == EOffsetRootBone_CollisionTestingMode::PlanarCollision)
		{
			float B = FVector::DotProduct(TranslationOffsetDelta, CollisionNormal);
			if (B > UE_KINDA_SMALL_NUMBER)
			{
				FVector OffsetCollisionPoint = CollisionPoint + CollisionNormal * GetCollisionTestShapeRadius();
				float CollisionParam = FVector::DotProduct((OffsetCollisionPoint - SimulatedTranslation), CollisionNormal) / B;
				if (CollisionParam >=0 && CollisionParam < 1)
				{
					FVector TranslationToPlane = TranslationOffsetDelta * CollisionParam;
					FVector TranslationAlongPlane = TranslationOffsetDelta - TranslationToPlane;
					TranslationAlongPlane = TranslationAlongPlane - FVector::DotProduct(TranslationAlongPlane, CollisionNormal);

					TranslationOffsetDelta = TranslationToPlane + TranslationAlongPlane;
				}
			}
		}
		

		SimulatedTranslation = SimulatedTranslation + TranslationOffsetDelta;
	}

	if (GetRotationMode() == EOffsetRootBoneMode::Release ||
		GetRotationMode() == EOffsetRootBoneMode::Interpolate)
	{
		FQuat RotationOffset = ComponentTransform.GetRotation() * SimulatedRotation.Inverse();
		RotationOffset.Normalize();
		RotationOffset = RotationOffset.W < 0.0 ? -RotationOffset : RotationOffset;

		const float DampenAlpha = UE::Anim::OffsetRootBone::DamperImplicit(GetRotationHalfLife(), CachedDeltaTime);
		FQuat RotationOffsetDelta = FQuat::Slerp(FQuat::Identity, RotationOffset, DampenAlpha);

		if (GetClampToRotationVelocity())
		{
			float RotationMotionAngleDelta;
			FVector RootMotionRotationAxis;
			RootMotionTransformDelta.GetRotation().ToAxisAndAngle(RootMotionRotationAxis, RotationMotionAngleDelta);

			float MaxRotationAngle = GetRotationSpeedRatio() * RotationMotionAngleDelta;

			FVector DeltaAxis;
			float DeltaAngle;
			RotationOffsetDelta.ToAxisAndAngle(DeltaAxis, DeltaAngle);

			if (DeltaAngle > MaxRotationAngle)
			{
				RotationOffsetDelta = FQuat(DeltaAxis, MaxRotationAngle);
			}
		}

		SimulatedRotation = RotationOffsetDelta * SimulatedRotation;
	}

	if (MaxTranslationOffset >= 0.0f)
	{
		FVector TranslationOffset = ComponentTransform.GetLocation() - SimulatedTranslation;
		const float TranslationOffsetSize = TranslationOffset.Size();
		if (TranslationOffsetSize > MaxTranslationOffset)
		{
			TranslationOffset = TranslationOffset.GetClampedToMaxSize(MaxTranslationOffset);
			SimulatedTranslation = ComponentTransform.GetLocation() - TranslationOffset;
		}
	}

	const float MaxAngleRadians = FMath::DegreesToRadians(GetMaxRotationError());
	if (GetMaxRotationError() >= 0.0f)
	{
		FQuat RotationOffset = ComponentTransform.GetRotation().Inverse() * SimulatedRotation;
		RotationOffset.Normalize();
		RotationOffset = RotationOffset.W < 0.0 ? -RotationOffset : RotationOffset;

		FVector OffsetAxis;
		float OffsetAngle;
		RotationOffset.ToAxisAndAngle(OffsetAxis, OffsetAngle);

		if (FMath::Abs(OffsetAngle) > MaxAngleRadians)
		{
			RotationOffset = FQuat(OffsetAxis, MaxAngleRadians);
			SimulatedRotation = RotationOffset * ComponentTransform.GetRotation();
			SimulatedRotation.Normalize();
		}
	}

	// Apply the offset adjustments to the simulated transform
	SimulatedTransform.SetLocation(SimulatedTranslation);
	SimulatedTransform.SetRotation(SimulatedRotation);

	// Start with the input pose's bone transform, to preserve any adjustments done before this node in the graph
	FTransform TargetBoneTransform = InputBoneTransform;
	// Accumulate the simulated transform in, and counter current component transform.
	TargetBoneTransform.Accumulate(SimulatedTransform * ComponentTransform.Inverse());

	// Offset root bone should not affect scale so take the input
	TargetBoneTransform.SetScale3D(InputBoneTransform.GetScale3D());

	TargetBoneTransform.NormalizeRotation();

	Output.Pose[TargetBoneIndex] = TargetBoneTransform;

#if ENABLE_VISUAL_LOG
	if (FVisualLogger::IsRecording())
	{
		static const TCHAR* LogName = TEXT("OffsetRootBone");
		const float InnerCircleRadius = 40.0f;
		const uint16 CircleThickness = 2;
		const FVector CircleOffset(0,0,1);

		const FTransform TargetBoneInitialTransformWorld = InputBoneTransform * ComponentTransform;
		const FTransform TargetBoneTransformWorld = TargetBoneTransform * ComponentTransform;
		UObject* LogOwner = AnimInstanceProxy->GetAnimInstanceObject();

		if (MaxTranslationOffset >= 0.0f)
		{
			const float OuterCircleRadius = MaxTranslationOffset + InnerCircleRadius;
			UE_VLOG_CIRCLE_THICK(AnimInstanceProxy->GetAnimInstanceObject(), TEXT("OffsetRootBone"), Display, ComponentTransform.GetLocation() + CircleOffset, FVector::UpVector, OuterCircleRadius, FColor::Red, CircleThickness, TEXT(""));
			
			if (bCollisionDetected)
			{
				UE_VLOG_CIRCLE_THICK(LogOwner, LogName, Display, CollisionPoint, CollisionNormal, GetCollisionTestShapeRadius(), FColor::Red, CircleThickness, TEXT(""));
			}
		}
		
		UE_VLOG_CIRCLE_THICK(LogOwner, LogName, Display, ComponentTransform.GetLocation() + CircleOffset, FVector::UpVector, InnerCircleRadius, FColor::Blue, CircleThickness, TEXT(""));
		UE_VLOG_ARROW(LogOwner, LogName, Display,
			ComponentTransform.GetLocation() + CircleOffset,
			ComponentTransform.GetLocation() + InnerCircleRadius * ComponentTransform.GetRotation().GetRightVector() + CircleOffset,
			FColor::Blue, TEXT(""));
		
		UE_VLOG_CIRCLE_THICK(LogOwner, LogName, Display, TargetBoneTransformWorld.GetLocation() + CircleOffset, FVector::UpVector, InnerCircleRadius, FColor::Green, CircleThickness, TEXT(""));
		UE_VLOG_ARROW(LogOwner, LogName, Display,
		 	TargetBoneTransformWorld.GetLocation() + CircleOffset,
		 	TargetBoneTransformWorld.GetLocation() + InnerCircleRadius * TargetBoneTransformWorld.GetRotation().GetRightVector() + CircleOffset,
			FColor::Green, TEXT(""));
	}
#endif

#if ENABLE_ANIM_DEBUG
	bool bDebugging = CVarAnimNodeOffsetRootBoneDebug.GetValueOnAnyThread() == 1;
	if (bDebugging)
	{
		const float InnerCircleRadius = 40.0f;
		const float CircleThickness = 1.5f;
		const float ConeThickness = 0.3f;

		const FTransform TargetBoneInitialTransformWorld = InputBoneTransform * ComponentTransform;
		const FTransform TargetBoneTransformWorld = TargetBoneTransform * ComponentTransform;

		if (MaxTranslationOffset >= 0.0f)
		{
			const float OuterCircleRadius = MaxTranslationOffset + InnerCircleRadius;
			AnimInstanceProxy->AnimDrawDebugCircle(ComponentTransform.GetLocation(), OuterCircleRadius, 36, FColor::Red,
				FVector::UpVector, false, -1.0f, SDPG_World, CircleThickness);
		}

		AnimInstanceProxy->AnimDrawDebugCircle(ComponentTransform.GetLocation(), InnerCircleRadius, 36, FColor::Blue,
			FVector::UpVector, false, -1.0f, SDPG_World, CircleThickness);
			
		AnimInstanceProxy->AnimDrawDebugCircle(TargetBoneTransformWorld.GetLocation(), InnerCircleRadius, 36, FColor::Green,
			FVector::UpVector, false, -1.0f, SDPG_World, CircleThickness);

		const int32 ConeSegments =  FMath::RoundUpToPowerOfTwo((GetMaxRotationError() / 180.0f) * 12 );
		const FVector ArcDirection = ComponentTransform.GetRotation().GetRightVector();
		AnimInstanceProxy->AnimDrawDebugCone(TargetBoneTransformWorld.GetLocation(), 0.9f * InnerCircleRadius, ArcDirection,
			MaxAngleRadians, 0.0f, ConeSegments, FColor::Red, false, -1.0f, SDPG_World, ConeThickness);

		AnimInstanceProxy->AnimDrawDebugDirectionalArrow(
			TargetBoneTransformWorld.GetLocation() + InnerCircleRadius * TargetBoneInitialTransformWorld.GetRotation().GetRightVector(),
			TargetBoneTransformWorld.GetLocation() + 1.5f * InnerCircleRadius * TargetBoneInitialTransformWorld.GetRotation().GetRightVector(),
			40.f, FColor::Red, false, 0.f, CircleThickness);

		AnimInstanceProxy->AnimDrawDebugDirectionalArrow(
			TargetBoneTransformWorld.GetLocation() + InnerCircleRadius * TargetBoneTransformWorld.GetRotation().GetRightVector(),
			TargetBoneTransformWorld.GetLocation() + 1.3f * InnerCircleRadius * TargetBoneTransformWorld.GetRotation().GetRightVector(),
			40.f, FColor::Blue, false, 0.f, CircleThickness);
	}
#endif


	if (bGraphDriven && bModifyBone)
	{
		const FTransform RemainingRootMotionDelta = ConsumedRootMotionDelta * RootMotionTransformDelta.Inverse();
		FTransform TargetRootMotionTransformDelta(RemainingRootMotionDelta.GetRotation(), RemainingRootMotionDelta.GetTranslation(), RootMotionTransformDelta.GetScale3D());
		const bool bRootMotionOverridden = RootMotionProvider->OverrideRootMotion(RootMotionTransformDelta, Output.CustomAttributes);
		ensure(bRootMotionOverridden);
	}

	bIsFirstUpdate = false;
}

EWarpingEvaluationMode FAnimNode_OffsetRootBone::GetEvaluationMode() const
{
	return GET_ANIM_NODE_DATA(EWarpingEvaluationMode, EvaluationMode);
}

bool FAnimNode_OffsetRootBone::GetResetEveryFrame() const
{
	return GET_ANIM_NODE_DATA(bool, bResetEveryFrame);
}

bool FAnimNode_OffsetRootBone::GetOnGround() const
{
	return GET_ANIM_NODE_DATA(bool, bOnGround);
}

const FVector& FAnimNode_OffsetRootBone::GetGroundNormal() const
{
	return GET_ANIM_NODE_DATA(FVector, GroundNormal);
}

const FVector& FAnimNode_OffsetRootBone::GetTranslationDelta() const
{
	return GET_ANIM_NODE_DATA(FVector, TranslationDelta);
}

const FRotator& FAnimNode_OffsetRootBone::GetRotationDelta() const
{
	return GET_ANIM_NODE_DATA(FRotator, RotationDelta);
}

EOffsetRootBoneMode FAnimNode_OffsetRootBone::GetTranslationMode() const
{
	return GET_ANIM_NODE_DATA(EOffsetRootBoneMode, TranslationMode);
}

EOffsetRootBoneMode FAnimNode_OffsetRootBone::GetRotationMode() const
{
	return GET_ANIM_NODE_DATA(EOffsetRootBoneMode, RotationMode);
}

float FAnimNode_OffsetRootBone::GetTranslationHalflife() const
{
	return GET_ANIM_NODE_DATA(float, TranslationHalflife);
}

float FAnimNode_OffsetRootBone::GetRotationHalfLife() const
{
	return GET_ANIM_NODE_DATA(float, RotationHalfLife);
}

float FAnimNode_OffsetRootBone::GetMaxTranslationError() const
{
	return GET_ANIM_NODE_DATA(float, MaxTranslationError);
}

float FAnimNode_OffsetRootBone::GetMaxRotationError() const
{
	return GET_ANIM_NODE_DATA(float, MaxRotationError);
}

bool FAnimNode_OffsetRootBone::GetClampToTranslationVelocity() const
{
	return GET_ANIM_NODE_DATA(bool, bClampToTranslationVelocity);
}

bool FAnimNode_OffsetRootBone::GetClampToRotationVelocity() const
{
	return GET_ANIM_NODE_DATA(bool, bClampToRotationVelocity);
}

float FAnimNode_OffsetRootBone::GetTranslationSpeedRatio() const
{
	return GET_ANIM_NODE_DATA(float, TranslationSpeedRatio);
}

float FAnimNode_OffsetRootBone::GetRotationSpeedRatio() const
{
	return GET_ANIM_NODE_DATA(float, RotationSpeedRatio);
}

EOffsetRootBone_CollisionTestingMode FAnimNode_OffsetRootBone::GetCollisionTestingMode() const
{
	return GET_ANIM_NODE_DATA(EOffsetRootBone_CollisionTestingMode, CollisionTestingMode);
}

float FAnimNode_OffsetRootBone::GetCollisionTestShapeRadius() const
{
	return GET_ANIM_NODE_DATA(float, CollisionTestShapeRadius);
}

const FVector& FAnimNode_OffsetRootBone::GetCollisionTestShapeOffset() const
{
	return GET_ANIM_NODE_DATA(FVector, CollisionTestShapeOffset);
}

bool FAnimNode_OffsetRootBone::GetResetOnTeleport() const
{
	return GET_ANIM_NODE_DATA(bool, bResetOnTeleport);
}

void FAnimNode_OffsetRootBone::Reset(const FAnimationBaseContext& Context)
{
	ComponentTransform = Context.AnimInstanceProxy->GetComponentTransform();
	SimulatedTranslation = ComponentTransform.GetLocation();
	SimulatedRotation = ComponentTransform.GetRotation();
	bIsFirstUpdate = true;
}
