// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_OrientationWarping.h"

#include "Animation/AnimBlendDebugScope.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimNodeFunctionRef.h"
#include "Animation/AnimRootMotionProvider.h"
#include "BoneControllers/AnimNode_OffsetRootBone.h"
#include "HAL/IConsoleManager.h"
#include "Animation/AnimTrace.h"
#include "Logging/LogVerbosity.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_OrientationWarping)

DECLARE_CYCLE_STAT(TEXT("OrientationWarping Eval"), STAT_OrientationWarping_Eval, STATGROUP_Anim);

#if ENABLE_ANIM_DEBUG
static int32 AnimNodeOrientationWarpingDebug = 0;
static FAutoConsoleVariableRef CVarAnimNodeOrientationWarpingDebug(TEXT("a.AnimNode.OrientationWarping.Debug"), AnimNodeOrientationWarpingDebug, TEXT("Turn on visualization debugging for Orientation Warping."));

static int32 AnimNodeOrientationWarpingVerbose = 0;
static FAutoConsoleVariableRef CVarAnimNodeOrientationWarpingVerbose(TEXT("a.AnimNode.OrientationWarping.Verbose"), AnimNodeOrientationWarpingVerbose, TEXT("Turn on verbose graph debugging for Orientation Warping"));

static int32 AnimNodeOrientationWarpingEnable = 1;
static FAutoConsoleVariableRef CVarAnimNodeOrientationWarpingEnable(TEXT("a.AnimNode.OrientationWarping.Enable"), AnimNodeOrientationWarpingEnable, TEXT("Toggle Orientation Warping"));
#endif

#if ENABLE_ANIM_DEBUG || ENABLE_VISUAL_LOG
static int32 AnimNodeOrientationWarpingDebugTransparency = 1;
static FAutoConsoleVariableRef CVarAnimNodeOrientationWarpingDebugTransparency(TEXT("a.AnimNode.OrientationWarping.Debug.Transparency"), AnimNodeOrientationWarpingDebugTransparency, TEXT("Enable transparency for debug draws, based on blend weight"));
#endif

namespace UE::Anim
{
	static inline FVector GetAxisVector(const EAxis::Type& InAxis)
	{
		switch (InAxis)
		{
		case EAxis::X:
			return FVector::ForwardVector;
		case EAxis::Y:
			return FVector::RightVector;
		default:
			return FVector::UpVector;
		};
	}

	static inline bool IsInvalidWarpingAngleDegrees(float Angle, float Tolerance)
	{
		Angle = FRotator::NormalizeAxis(Angle);
		return FMath::IsNearlyZero(Angle, Tolerance) || FMath::IsNearlyEqual(FMath::Abs(Angle), 180.f, Tolerance);
	}

	static float SignedAngleRadBetweenNormals(const FVector& From, const FVector& To, const FVector& Axis)
	{
		const float FromDotTo = FVector::DotProduct(From, To);
		const float Angle = FMath::Acos(FromDotTo);
		const FVector Cross = FVector::CrossProduct(From, To);
		const float Dot = FVector::DotProduct(Cross, Axis);
		return Dot >= 0 ? Angle : -Angle;
	}
}

void FAnimNode_OrientationWarping::GatherDebugData(FNodeDebugData& DebugData)
{
	FString DebugLine = DebugData.GetNodeName(this);
#if ENABLE_ANIM_DEBUG
	if (AnimNodeOrientationWarpingVerbose == 1)
	{
		if (Mode == EWarpingEvaluationMode::Manual)
		{
			DebugLine += TEXT("\n - Evaluation Mode: (Manual)");
			DebugLine += FString::Printf(TEXT("\n - Orientation Angle: (%.3fd)"), FMath::RadiansToDegrees(ActualOrientationAngleRad));
		}
		else
		{
			DebugLine += TEXT("\n - Evaluation Mode: (Graph)");
			DebugLine += FString::Printf(TEXT("\n - Orientation Angle: (%.3fd)"), FMath::RadiansToDegrees(ActualOrientationAngleRad));
			// Locomotion angle is already in degrees.
			DebugLine += FString::Printf(TEXT("\n - Locomotion Angle: (%.3fd)"), LocomotionAngle);
			DebugLine += FString::Printf(TEXT("\n - Locomotion Delta Angle Threshold: (%.3fd)"), LocomotionAngleDeltaThreshold);
#if WITH_EDITORONLY_DATA
			DebugLine += FString::Printf(TEXT("\n - Root Motion Delta Attribute Found: %s)"), (bFoundRootMotionAttribute) ? TEXT("true") : TEXT("false"));
#endif
		}
		if (const UEnum* TypeEnum = FindObject<UEnum>(nullptr, TEXT("/Script/CoreUObject.EAxis")))
		{
			DebugLine += FString::Printf(TEXT("\n - Rotation Axis: (%s)"), *(TypeEnum->GetNameStringByIndex(static_cast<int32>(RotationAxis))));
		}
		DebugLine += FString::Printf(TEXT("\n - Rotation Interpolation Speed: (%.3fd)"), RotationInterpSpeed);
	}
	else
#endif
	{
	const float ActualOrientationAngleDegrees = FMath::RadiansToDegrees(ActualOrientationAngleRad);
	DebugLine += FString::Printf(TEXT("(Orientation Angle: %.3fd)"), ActualOrientationAngleDegrees);
	}
	DebugData.AddDebugItem(DebugLine);
	ComponentPose.GatherDebugData(DebugData);
}

void FAnimNode_OrientationWarping::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	FAnimNode_SkeletalControlBase::Initialize_AnyThread(Context);

	Reset(Context);
}

void FAnimNode_OrientationWarping::UpdateInternal(const FAnimationUpdateContext& Context)
{
	FAnimNode_SkeletalControlBase::UpdateInternal(Context);

	// If we just became relevant and haven't been initialized yet, then reset.
	if (!bIsFirstUpdate && UpdateCounter.HasEverBeenUpdated() && !UpdateCounter.WasSynchronizedCounter(Context.AnimInstanceProxy->GetUpdateCounter()))
	{
		Reset(Context);
	}
	UpdateCounter.SynchronizeWith(Context.AnimInstanceProxy->GetUpdateCounter());
	BlendWeight = Context.GetFinalBlendWeight();

#if ENABLE_ANIM_DEBUG || ENABLE_VISUAL_LOG
	UE::Anim::FAnimBlendDebugScope* BlendDebugMessage = Context.GetMessage<UE::Anim::FAnimBlendDebugScope>();
	if (BlendDebugMessage)
	{
		DebugColor = BlendDebugMessage->DebugColor;
	}
#endif

	if (WarpingSpace == EOrientationWarpingSpace::RootBoneTransform)
	{
		if (UE::AnimationWarping::FRootOffsetProvider* RootOffsetProvider = Context.GetMessage<UE::AnimationWarping::FRootOffsetProvider>())
		{
			WarpingSpaceTransform = RootOffsetProvider->GetRootTransform();
		}
		else
		{
			WarpingSpaceTransform = Context.AnimInstanceProxy->GetComponentTransform();
		}
	}
}

void FAnimNode_OrientationWarping::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	SCOPE_CYCLE_COUNTER(STAT_OrientationWarping_Eval);
	check(OutBoneTransforms.Num() == 0);

	float TargetOrientationAngleRad;

	const float DeltaSeconds = Output.AnimInstanceProxy->GetDeltaSeconds();
	const float MaxAngleCorrectionRad = FMath::DegreesToRadians(MaxCorrectionDegrees);
	const FVector RotationAxisVector = UE::Anim::GetAxisVector(RotationAxis);
	FVector LocomotionForward = FVector::ZeroVector;

	bool bGraphDrivenWarping = false;
	const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get();

	if (Mode == EWarpingEvaluationMode::Graph)
	{
		bGraphDrivenWarping = !!RootMotionProvider;
		if (!bGraphDrivenWarping)
		{
			UE_LOGF(LogAnimation, Error, "FAnimNode_OrientationWarping, Graph-driven mode expected a valid root motion delta provider interface");
			return;
		}
	}

#if WITH_EDITORONLY_DATA
	bFoundRootMotionAttribute = false;
#endif

#if ENABLE_ANIM_DEBUG || ANIM_TRACE_ENABLED
	FTransform RootMotionTransformDelta = FTransform::Identity;
	float RootMotionDeltaAngleRad = 0.0;
	const float PreviousOrientationAngleRad = ActualOrientationAngleRad;
#endif

#if ENABLE_ANIM_DEBUG || ENABLE_VISUAL_LOG
	if (DeltaSeconds > 0)
	{
		bUsedFutureRootMotion = false;
	}
#endif

	// We will likely need to revisit LocomotionAngle participating as an input to orientation warping.
	// Without velocity information from the motion model (such as the capsule), LocomotionAngle isn't enough
	// information in isolation for all cases when deciding to warp.
	//
	// For example imagine that the motion model has stopped moving with zero velocity due to a
	// transition into a strafing stop. During that transition we may play an animation with non-zero 
	// velocity for an arbitrary number of frames. In this scenario the concept of direction is meaningless 
	// since we cannot orient the animation to match a zero velocity and consequently a zero direction, 
	// since that would break the pose. For those frames, we would incorrectly over-orient the strafe.
	//
	// The solution may be instead to pass velocity with the actor base rotation, allowing us to retain
	// speed information about the motion. It may also allow us to do more complex orienting behavior 
	// when multiple degrees of freedom can be considered.

	if (WarpingSpace == EOrientationWarpingSpace::ComponentTransform)
	{
		WarpingSpaceTransform = Output.AnimInstanceProxy->GetComponentTransform();
	}

	if (bGraphDrivenWarping)
	{
#if !ENABLE_ANIM_DEBUG && !ANIM_TRACE_ENABLED
		FTransform RootMotionTransformDelta = FTransform::Identity;
#endif

		bGraphDrivenWarping = RootMotionProvider->ExtractRootMotion(Output.CustomAttributes, RootMotionTransformDelta);

		// Graph driven orientation warping will modify the incoming root motion to orient towards the intended locomotion angle
		if (bGraphDrivenWarping)
		{
#if WITH_EDITORONLY_DATA
			// Graph driven Orientation Warping expects a root motion delta to be present in the attribute stream.
			bFoundRootMotionAttribute = true;
#endif

			// In UE, forward is defined as +x; consequently this is also true when sampling an actor's velocity. Historically the skeletal 
			// mesh component forward will not match the actor, requiring us to correct the rotation before sampling the LocomotionForward.
			// In order to make orientation warping 'pure' in the future we will need to provide more context about the intent of
			// the actor vs the intent of the animation in their respective spaces. Specifically, we will need some form the following information:
			//
			// 1. Actor Forward
			// 2. Actor Velocity
			// 3. Skeletal Mesh Relative Rotation

			if (LocomotionDirection.SquaredLength() > UE_SMALL_NUMBER)
			{
				// if we have a LocomotionDirection vector, transform into root bone local space
				LocomotionForward = WarpingSpaceTransform.InverseTransformVector(LocomotionDirection);
				LocomotionForward.Normalize();
			}
			else
			{
				LocomotionAngle = FRotator::NormalizeAxis(LocomotionAngle);
				// UE-184297 Avoid storing LocomotionAngle in radians in case haven't updated the pinned input, to avoid a DegToRad(RadianValue)
				const float LocomotionAngleRadians = FMath::DegreesToRadians(LocomotionAngle);
				const FQuat LocomotionRotation = FQuat(RotationAxisVector, LocomotionAngleRadians);
				const FTransform SkeletalMeshRelativeTransform = Output.AnimInstanceProxy->GetComponentRelativeTransform();
            	const FQuat SkeletalMeshRelativeRotation = SkeletalMeshRelativeTransform.GetRotation();
				LocomotionForward = SkeletalMeshRelativeRotation.UnrotateVector(LocomotionRotation.GetForwardVector()).GetSafeNormal();
			}

			// Flatten locomotion direction, along the rotation axis.
			LocomotionForward = (LocomotionForward - RotationAxisVector.Dot(LocomotionForward) * RotationAxisVector).GetSafeNormal();

			// @todo: Graph mode using a "manual value" makes no sense. Restructure logic to address this in the future.
			if (bUseManualRootMotionVelocity)
			{
				RootMotionTransformDelta.SetTranslation(ManualRootMotionVelocity * DeltaSeconds);
			}

			FVector RootMotionDeltaTranslation = RootMotionTransformDelta.GetTranslation();
			const FQuat PreviousRootMotionDeltaRotation = RootMotionDeltaRotation;
			RootMotionDeltaRotation = RootMotionTransformDelta.GetRotation();
			
			// Flatten root motion translation, along the rotation axis.
			RootMotionDeltaTranslation = RootMotionDeltaTranslation - RotationAxisVector.Dot(RootMotionDeltaTranslation) * RotationAxisVector;

			// Look forward in root motion to determine orientation warping angle if future time specified
			FVector PredictedRootMotionDeltaTranslation = RootMotionDeltaTranslation;
			if (TargetTime > 0 && CurrentAnimAsset)
			{
				if (UAnimSequenceBase* AnimSequence = Cast<UAnimSequenceBase>(CurrentAnimAsset))
				{
					const FAnimExtractContext Context(static_cast<double>(CurrentAnimAssetTime), true, FDeltaTimeRecord(TargetTime), false);
					FTransform PredictedRootMotionTransform = AnimSequence->ExtractRootMotion(Context);
					PredictedRootMotionDeltaTranslation = PredictedRootMotionTransform.GetTranslation();
					PredictedRootMotionDeltaTranslation = PredictedRootMotionDeltaTranslation - RotationAxisVector.Dot(PredictedRootMotionDeltaTranslation) * RotationAxisVector;
				}
			}
			
			const float RootMotionDeltaSpeed = RootMotionDeltaTranslation.Size() / DeltaSeconds;
			if (RootMotionDeltaSpeed < MinRootMotionSpeedThreshold)
			{
				// If we're under the threshold, snap orientation angle to 0, and let interpolation handle the delta
				TargetOrientationAngleRad = 0.0f;
			}
			else
			{
				const FVector PreviousRootMotionDeltaDirection = RootMotionDeltaDirection;
				// Hold previous direction if we can't calculate it from current move delta, because the root is no longer moving
				RootMotionDeltaDirection = RootMotionDeltaTranslation.GetSafeNormal(UE_SMALL_NUMBER, PreviousRootMotionDeltaDirection);
				TargetOrientationAngleRad = UE::Anim::SignedAngleRadBetweenNormals(RootMotionDeltaDirection, LocomotionForward, RotationAxisVector);

				// Motion Matching may return an animation that deviates a lot from the movement direction (e.g movement direction going bwd and motion matching could return the fwd animation for a few frames)
				// When that happens, since we use the delta between root motion and movement direction, we would be over-rotating the lower body and breaking the pose during those frames
				// So, when that happens we use the inverse of the root motion direction to calculate our target rotation. 
				// This feels a bit 'hacky' but its the only option I've found so far to mitigate the problem
				if (LocomotionAngleDeltaThreshold > 0.f)
				{
					if (FMath::Abs(FMath::RadiansToDegrees(TargetOrientationAngleRad)) > LocomotionAngleDeltaThreshold)
					{
						TargetOrientationAngleRad = FMath::UnwindRadians(TargetOrientationAngleRad + FMath::DegreesToRadians(180.0f));
						RootMotionDeltaDirection = -RootMotionDeltaDirection;
					}
				}

				// If there is translation in predicted root motion, use it if the orientation error is less than current error
				if (TargetTime > 0 && CurrentAnimAsset && !PredictedRootMotionDeltaTranslation.IsNearlyZero(UE_SMALL_NUMBER))
				{
					PredictedRootMotionDeltaTranslation.Normalize();
					float PredictedOrientationErrorAngleRad = UE::Anim::SignedAngleRadBetweenNormals(PredictedRootMotionDeltaTranslation, LocomotionForward, RotationAxisVector);

					// The future orientation will often match the current, so add a small delta to avoid further testing for same values
					if (FMath::Abs(PredictedOrientationErrorAngleRad) + UE_KINDA_SMALL_NUMBER < FMath::Abs(TargetOrientationAngleRad))
					{
						// Note: Don't update root motion direction, as the root motion direction is what we are playing, 
						// not the future. Updating root motion direction will break counter compensate. 
						// Also, we rely on the built in interp for continuity / smoothness
#if ENABLE_ANIM_DEBUG || ENABLE_VISUAL_LOG
						FutureRootMotionDeltaDirection = PredictedRootMotionDeltaTranslation;
						bUsedFutureRootMotion = true;
#endif
						TargetOrientationAngleRad = PredictedOrientationErrorAngleRad;
					}
				}

				// Don't compensate interpolation by the root motion angle delta if the previous direction isn't valid.
				if (bCounterCompenstateInterpolationByRootMotion && !PreviousRootMotionDeltaDirection.IsNearlyZero(UE_SMALL_NUMBER))
				{
#if !ENABLE_ANIM_DEBUG && !ANIM_TRACE_ENABLED
					float RootMotionDeltaAngleRad;
#endif
					// Counter the interpolated orientation angle by the root motion direction angle delta.
					// This prevents our interpolation from fighting the natural root motion that's flowing through the graph.
					// To correctly measure the amount to counter, we need to unrotate our previous delta direction by our previous rotation
					// As the previous direction delta is relative to the previous rotation delta
					RootMotionDeltaAngleRad = UE::Anim::SignedAngleRadBetweenNormals(RootMotionDeltaDirection, PreviousRootMotionDeltaRotation.UnrotateVector(PreviousRootMotionDeltaDirection), RotationAxisVector);
					
					// Root motion may have large deltas i.e. bad blends or sudden direction changes like pivots.
					// If there's an instantaneous pop in root motion direction, this is likely a pivot.
					const float MaxRootMotionDeltaToCompensateRad = FMath::DegreesToRadians(MaxRootMotionDeltaToCompensateDegrees);
					if (FMath::Abs(RootMotionDeltaAngleRad) < MaxRootMotionDeltaToCompensateRad)
					{
						CounterCompensateTargetAngleRad += RootMotionDeltaAngleRad;
						float CounterCompensateAngle = FMath::FInterpTo(0, CounterCompensateTargetAngleRad, DeltaSeconds, CounterCompensateInterpSpeed);
						ActualOrientationAngleRad = FMath::UnwindRadians(ActualOrientationAngleRad + CounterCompensateAngle);
						CounterCompensateTargetAngleRad -= CounterCompensateAngle;
					}
				}

				// Rotate the root motion delta fully by the warped angle
				const FVector WarpedRootMotionTranslationDelta = FQuat(RotationAxisVector, TargetOrientationAngleRad).RotateVector(RootMotionDeltaTranslation);
				RootMotionTransformDelta.SetTranslation(WarpedRootMotionTranslationDelta);
			}

			// Forward the side effects of orientation warping on the root motion contribution for this sub-graph
			const bool bRootMotionOverridden = RootMotionProvider->OverrideRootMotion(RootMotionTransformDelta, Output.CustomAttributes);
			if (!bRootMotionOverridden)
			{
				UE_LOGF(LogAnimation, Error, "FAnimNode_OrientationWarping, Graph-driven mode expected a root motion delta to be present in the attribute stream prior to warping/overriding it");
				return;
			}
		}
		else
		{
			// Early exit on missing root motion delta attribute
			return;
		}
	} 
	else
	{
		// Manual orientation warping will take the angle directly
		TargetOrientationAngleRad = FRotator::NormalizeAxis(OrientationAngle);
		TargetOrientationAngleRad = FMath::DegreesToRadians(TargetOrientationAngleRad);
	}

	// Optionally interpolate the effective orientation towards the target orientation angle
	// When the orientation warping node becomes relevant, the input pose orientation may not be aligned with the desired orientation.
	// Instead of interpolating this difference, snap to the desired orientation if it's our first update to minimize corrections over-time.
	if ((RotationInterpSpeed > 0.f) && !bIsFirstUpdate)
	{
		const float SmoothOrientationAngleRad = FMath::FInterpTo(ActualOrientationAngleRad, TargetOrientationAngleRad, DeltaSeconds, RotationInterpSpeed);
		// Limit our interpolation rate to prevent pops.
		// @TODO: Use better, more physically accurate interpolation here.
		ActualOrientationAngleRad = FMath::Clamp(SmoothOrientationAngleRad, ActualOrientationAngleRad - MaxAngleCorrectionRad, ActualOrientationAngleRad + MaxAngleCorrectionRad);
	}
	else
	{
		ActualOrientationAngleRad = TargetOrientationAngleRad;
	}

	ActualOrientationAngleRad = FMath::Clamp(ActualOrientationAngleRad, -MaxAngleCorrectionRad, MaxAngleCorrectionRad);
	// Allow the alpha value of the node to affect the final rotation
	ActualOrientationAngleRad *= ActualAlpha;

	if (bScaleByGlobalBlendWeight)
	{
		ActualOrientationAngleRad *= BlendWeight;
	}

#if ENABLE_ANIM_DEBUG
	bool bDebugging = false;
#if WITH_EDITORONLY_DATA
	bDebugging = bDebugging || bEnableDebugDraw;
#else
	constexpr float DebugDrawScale = 1.f;
#endif
	const int32 DebugIndex = AnimNodeOrientationWarpingDebug;
	bDebugging = bDebugging || (DebugIndex > 0);

	if (bDebugging)
	{
		const FTransform ComponentTransform = Output.AnimInstanceProxy->GetComponentTransform();
		const FVector ActorForwardDirection = Output.AnimInstanceProxy->GetActorTransform().GetRotation().GetForwardVector();
		FVector DebugArrowOffset = FVector::ZAxisVector * DebugDrawScale;
		uint8 DebugAlpha = AnimNodeOrientationWarpingDebugTransparency ? 255 * BlendWeight : 255;

		// Draw debug shapes
		{
			const FVector ForwardDirection = bGraphDrivenWarping
				? ComponentTransform.GetRotation().RotateVector(LocomotionForward)
				: ActorForwardDirection;

			Output.AnimInstanceProxy->AnimDrawDebugCircle(
				ComponentTransform.GetLocation() + DebugArrowOffset + ForwardDirection * 100.f * DebugDrawScale,
				4.f * DebugDrawScale, 8, DebugColor.WithAlpha(DebugAlpha), FVector::UpVector, false, 0.f, SDPG_World, 1.f);
			Output.AnimInstanceProxy->AnimDrawDebugDirectionalArrow(
				ComponentTransform.GetLocation() + DebugArrowOffset,
				ComponentTransform.GetLocation() + DebugArrowOffset + ForwardDirection * 100.f * DebugDrawScale,
				40.f * DebugDrawScale, FColor::Red, false, 0.f, 2.f * DebugDrawScale);

			const FVector RotationDirection = bGraphDrivenWarping
				? ComponentTransform.GetRotation().RotateVector(RootMotionDeltaDirection)
				: ActorForwardDirection.RotateAngleAxis(OrientationAngle, RotationAxisVector);

			DebugArrowOffset += FVector::ZAxisVector * DebugDrawScale;

			Output.AnimInstanceProxy->AnimDrawDebugCircle(
				ComponentTransform.GetLocation() + DebugArrowOffset + RotationDirection * 100.f * DebugDrawScale,
				4.f * DebugDrawScale, 8, DebugColor.WithAlpha(DebugAlpha), FVector::UpVector, false, 0.f, SDPG_World, 1.f);
			Output.AnimInstanceProxy->AnimDrawDebugDirectionalArrow(
				ComponentTransform.GetLocation() + DebugArrowOffset,
				ComponentTransform.GetLocation() + DebugArrowOffset + RotationDirection * 100.f * DebugDrawScale,
				40.f * DebugDrawScale, FColor::Blue.WithAlpha(DebugAlpha), false, 0.f, 2.f * DebugDrawScale);

			if (bUsedFutureRootMotion && bGraphDrivenWarping)
			{
				const FVector FutureRotationDirection = ComponentTransform.GetRotation().RotateVector(FutureRootMotionDeltaDirection);

				Output.AnimInstanceProxy->AnimDrawDebugCircle(
					ComponentTransform.GetLocation() + DebugArrowOffset + FutureRotationDirection * 100.f * DebugDrawScale,
					4.f * DebugDrawScale, 8, DebugColor.WithAlpha(DebugAlpha), FVector::UpVector, false, 0.f, SDPG_World, 1.f);
				Output.AnimInstanceProxy->AnimDrawDebugDirectionalArrow(
					ComponentTransform.GetLocation() + DebugArrowOffset,
					ComponentTransform.GetLocation() + DebugArrowOffset + FutureRotationDirection * 100.f * DebugDrawScale,
					40.f * DebugDrawScale, FColor::Yellow.WithAlpha(DebugAlpha), false, 0.f, 2.f * DebugDrawScale);
			}

			const float ActualOrientationAngleDegrees = FMath::RadiansToDegrees(ActualOrientationAngleRad);
			const FVector WarpedRotationDirection = bGraphDrivenWarping
				? RotationDirection.RotateAngleAxis(ActualOrientationAngleDegrees, RotationAxisVector)
				: ActorForwardDirection.RotateAngleAxis(ActualOrientationAngleDegrees, RotationAxisVector);

			DebugArrowOffset += FVector::ZAxisVector * DebugDrawScale;
			Output.AnimInstanceProxy->AnimDrawDebugCircle(
				ComponentTransform.GetLocation() + DebugArrowOffset + WarpedRotationDirection * 100.f * DebugDrawScale,
				4.f * DebugDrawScale, 8, DebugColor.WithAlpha(DebugAlpha), FVector::UpVector, false, 0.f, SDPG_World, 1.f);
			Output.AnimInstanceProxy->AnimDrawDebugDirectionalArrow(
				ComponentTransform.GetLocation() + DebugArrowOffset,
				ComponentTransform.GetLocation() + DebugArrowOffset + WarpedRotationDirection * 100.f * DebugDrawScale,
				40.f * DebugDrawScale, FColor::Green.WithAlpha(DebugAlpha), false, 0.f, 2.f * DebugDrawScale);
		}

		// Draw text on mesh in world space
		{
			TStringBuilder<1024> DebugLine;

			const float PreviousOrientationAngleDegrees = FMath::RadiansToDegrees(PreviousOrientationAngleRad);
			const float ActualOrientationAngleDegrees = FMath::RadiansToDegrees(ActualOrientationAngleRad);
			const float TargetOrientationAngleDegrees = FMath::RadiansToDegrees(TargetOrientationAngleRad);
			if (Mode == EWarpingEvaluationMode::Manual)
			{
				DebugLine.Appendf(TEXT("\n - Previous Orientation Angle: (%.3fd)"), PreviousOrientationAngleDegrees);
				DebugLine.Appendf(TEXT("\n - Orientation Angle: (%.3fd)"), ActualOrientationAngleDegrees);
				DebugLine.Appendf(TEXT("\n - Target Orientation Angle: (%.3fd)"), TargetOrientationAngleRad);
			}
			else
			{
				if (RotationInterpSpeed > 0.0f)
				{
					DebugLine.Appendf(TEXT("\n - Previous Orientation Angle: (%.3fd)"), FMath::RadiansToDegrees(PreviousOrientationAngleRad));
					DebugLine.Appendf(TEXT("\n - Root Motion Frame Delta Angle: (%.3fd)"), FMath::RadiansToDegrees(RootMotionDeltaAngleRad));
				}
				DebugLine.Appendf(TEXT("\n - Actual Orientation Angle: (%.3fd)"), FMath::RadiansToDegrees(ActualOrientationAngleRad));
				DebugLine.Appendf(TEXT("\n - Target Orientation Angle: (%.3fd)"), FMath::RadiansToDegrees(TargetOrientationAngleRad));
				// Locomotion angle is already in degrees.
				DebugLine.Appendf(TEXT("\n - Locomotion Angle: (%.3fd)"), LocomotionAngle);
				DebugLine.Appendf(TEXT("\n - Root Motion Delta: %s)"), *RootMotionTransformDelta.GetTranslation().ToString());
				DebugLine.Appendf(TEXT("\n - Root Motion Speed: %.3fd)"), RootMotionTransformDelta.GetTranslation().Size() / DeltaSeconds);
			}
			Output.AnimInstanceProxy->AnimDrawDebugInWorldMessage(DebugLine.ToString(), FVector::UpVector * 50.0f, FColor::Yellow, 1.f /*TextScale*/);
		}
	}
#endif

#if ANIM_TRACE_ENABLED
	{
		const float PreviousOrientationAngleDegrees = FMath::RadiansToDegrees(PreviousOrientationAngleRad);
		const float ActualOrientationAngleDegrees = FMath::RadiansToDegrees(ActualOrientationAngleRad);
		const float TargetOrientationAngleDegrees = FMath::RadiansToDegrees(TargetOrientationAngleRad);

		TRACE_ANIM_NODE_VALUE(Output, TEXT("Previous OrientationAngle Degrees"), PreviousOrientationAngleDegrees);
		TRACE_ANIM_NODE_VALUE(Output, TEXT("Actual Orientation Angle Degrees"), ActualOrientationAngleDegrees);
		TRACE_ANIM_NODE_VALUE(Output, TEXT("Target Orientation Angle Degrees"), TargetOrientationAngleDegrees);

		if (Mode == EWarpingEvaluationMode::Graph)
		{
			const float RootMotionDeltaAngleDegrees = FMath::RadiansToDegrees(RootMotionDeltaAngleRad);
			TRACE_ANIM_NODE_VALUE(Output, TEXT("Root Motion Delta Angle Degrees"), RootMotionDeltaAngleDegrees);
			TRACE_ANIM_NODE_VALUE(Output, TEXT("Locomotion Angle"), LocomotionAngle);
			TRACE_ANIM_NODE_VALUE(Output, TEXT("Root Motion Translation Delta"), RootMotionTransformDelta.GetTranslation());

			const float RootMotionSpeed = RootMotionTransformDelta.GetTranslation().Size() / DeltaSeconds;
			TRACE_ANIM_NODE_VALUE(Output, TEXT("Root Motion Speed"), RootMotionSpeed);
		}
	}
#endif
	

#if ENABLE_VISUAL_LOG
	if (FVisualLogger::IsRecording())
	{
		const FTransform ComponentTransform = Output.AnimInstanceProxy->GetComponentTransform();
		const FVector ActorForwardDirection = Output.AnimInstanceProxy->GetActorTransform().GetRotation().GetForwardVector();
		FVector DebugArrowOffset = FVector::ZAxisVector * DebugDrawScale;
		uint8 DebugAlpha = AnimNodeOrientationWarpingDebugTransparency ? 255 * BlendWeight : 255;

		// Draw debug shapes
		{
			const FVector ForwardDirection = bGraphDrivenWarping
				? ComponentTransform.GetRotation().RotateVector(LocomotionForward)
				: ActorForwardDirection;

			UE_VLOG_CIRCLE_THICK(Output.AnimInstanceProxy->GetAnimInstanceObject(), "OrientationWarping", Display,
				ComponentTransform.GetLocation() + DebugArrowOffset + ForwardDirection * 100.f * DebugDrawScale,
				FVector::UpVector, 4.f * DebugDrawScale, DebugColor.WithAlpha(DebugAlpha), 1.0f, TEXT(""));
			UE_VLOG_ARROW(Output.AnimInstanceProxy->GetAnimInstanceObject(), "OrientationWarping", Display,
				ComponentTransform.GetLocation() + DebugArrowOffset,
				ComponentTransform.GetLocation() + DebugArrowOffset + ForwardDirection * 100.f * DebugDrawScale,
				FColor::Red.WithAlpha(DebugAlpha), TEXT(""));

			const FVector RotationDirection = bGraphDrivenWarping
				? ComponentTransform.GetRotation().RotateVector(RootMotionDeltaDirection)
				: ActorForwardDirection.RotateAngleAxis(OrientationAngle, RotationAxisVector);

			DebugArrowOffset += FVector::ZAxisVector * DebugDrawScale;

			UE_VLOG_CIRCLE_THICK(Output.AnimInstanceProxy->GetAnimInstanceObject(), "OrientationWarping", Display,
				ComponentTransform.GetLocation() + DebugArrowOffset + RotationDirection * 100.f * DebugDrawScale,
				FVector::UpVector, 4.f * DebugDrawScale, DebugColor.WithAlpha(DebugAlpha), 1.0f, TEXT(""));
			UE_VLOG_ARROW(Output.AnimInstanceProxy->GetAnimInstanceObject(), "OrientationWarping", Display,
				ComponentTransform.GetLocation() + DebugArrowOffset,
				ComponentTransform.GetLocation() + DebugArrowOffset + RotationDirection * 100.f * DebugDrawScale,
				FColor::Blue.WithAlpha(DebugAlpha), TEXT(""));

			if (bUsedFutureRootMotion && bGraphDrivenWarping)
			{
				const FVector FutureRotationDirection = ComponentTransform.GetRotation().RotateVector(FutureRootMotionDeltaDirection);

				UE_VLOG_CIRCLE_THICK(Output.AnimInstanceProxy->GetAnimInstanceObject(), "OrientationWarping", Display,
					ComponentTransform.GetLocation() + DebugArrowOffset + FutureRotationDirection * 100.f * DebugDrawScale,
					FVector::UpVector, 4.f * DebugDrawScale, DebugColor.WithAlpha(DebugAlpha), 1.0f, TEXT(""));
				UE_VLOG_ARROW(Output.AnimInstanceProxy->GetAnimInstanceObject(), "OrientationWarping", Display,
					ComponentTransform.GetLocation() + DebugArrowOffset,
					ComponentTransform.GetLocation() + DebugArrowOffset + FutureRotationDirection * 100.f * DebugDrawScale,
					FColor::Yellow.WithAlpha(DebugAlpha), TEXT(""));
			}

			const float ActualOrientationAngleDegrees = FMath::RadiansToDegrees(ActualOrientationAngleRad);
			const FVector WarpedRotationDirection = bGraphDrivenWarping
				? RotationDirection.RotateAngleAxis(ActualOrientationAngleDegrees, RotationAxisVector)
				: ActorForwardDirection.RotateAngleAxis(ActualOrientationAngleDegrees, RotationAxisVector);

			DebugArrowOffset += FVector::ZAxisVector * DebugDrawScale;

			UE_VLOG_CIRCLE_THICK(Output.AnimInstanceProxy->GetAnimInstanceObject(), "OrientationWarping", Display,
				ComponentTransform.GetLocation() + DebugArrowOffset + WarpedRotationDirection * 100.f * DebugDrawScale,
				FVector::UpVector, 4.f * DebugDrawScale, DebugColor.WithAlpha(DebugAlpha), 1.0f, TEXT(""));
			UE_VLOG_ARROW(Output.AnimInstanceProxy->GetAnimInstanceObject(), "OrientationWarping", Display,
				ComponentTransform.GetLocation() + DebugArrowOffset,
				ComponentTransform.GetLocation() + DebugArrowOffset + WarpedRotationDirection * 100.f * DebugDrawScale,
				FColor::Green.WithAlpha(DebugAlpha), TEXT(""));
		}
	}
#endif

	// Validate scalar inputs that could produce NaN quats downstream.
	if (!FMath::IsFinite(ActualOrientationAngleRad))
	{
		UE_LOGF(LogAnimation, Error, "FAnimNode_OrientationWarping, ActualOrientationAngleRad is non-finite (%f); falling back to 0", ActualOrientationAngleRad);
		ActualOrientationAngleRad = 0.f;
	}
	if (!FMath::IsFinite(DistributedBoneOrientationAlpha))
	{
		UE_LOGF(LogAnimation, Error, "FAnimNode_OrientationWarping, DistributedBoneOrientationAlpha is non-finite (%f); clamping to [0,1]", DistributedBoneOrientationAlpha);
		DistributedBoneOrientationAlpha = 0.5f;
	}
	DistributedBoneOrientationAlpha = FMath::Clamp(DistributedBoneOrientationAlpha, 0.f, 1.f);

	const float RootOffset = FMath::UnwindRadians(ActualOrientationAngleRad * DistributedBoneOrientationAlpha);
	
	// Rotate Root Bone first, as that cheaply rotates the whole pose with one transformation.
	if (!FMath::IsNearlyZero(RootOffset, KINDA_SMALL_NUMBER))
	{
		const FQuat RootRotation = FQuat(RotationAxisVector, RootOffset);
		const FCompactPoseBoneIndex RootBoneIndex(0);
	
		FTransform RootBoneTransform(Output.Pose.GetComponentSpaceTransform(RootBoneIndex));
		RootBoneTransform.SetRotation(RootRotation * RootBoneTransform.GetRotation());
		RootBoneTransform.NormalizeRotation();
		Output.Pose.SetComponentSpaceTransform(RootBoneIndex, RootBoneTransform);
	}

	const int32 NumSpineBones = SpineBoneDataArray.Num();
	const bool bSpineOrientationAlpha = !FMath::IsNearlyZero(DistributedBoneOrientationAlpha, KINDA_SMALL_NUMBER);
	const bool bUpdateSpineBones = (NumSpineBones > 0) && bSpineOrientationAlpha;

	if (bUpdateSpineBones)
	{
		// Spine bones counter rotate body orientation evenly across all bones.
		for (int32 ArrayIndex = 0; ArrayIndex < NumSpineBones; ArrayIndex++)
		{
			const FOrientationWarpingSpineBoneData& BoneData = SpineBoneDataArray[ArrayIndex];
			const FQuat SpineBoneCounterRotation = FQuat(RotationAxisVector, -ActualOrientationAngleRad * DistributedBoneOrientationAlpha * BoneData.Weight);
			check(BoneData.Weight > 0.f);

			FTransform SpineBoneTransform(Output.Pose.GetComponentSpaceTransform(BoneData.BoneIndex));
			SpineBoneTransform.SetRotation((SpineBoneCounterRotation * SpineBoneTransform.GetRotation()));
			SpineBoneTransform.NormalizeRotation();
			Output.Pose.SetComponentSpaceTransform(BoneData.BoneIndex, SpineBoneTransform);
		}
	}

	const float IKFootRootOrientationAlpha = 1.f - DistributedBoneOrientationAlpha;
	const bool bUpdateIKFootRoot = (IKFootData.IKFootRootBoneIndex != FCompactPoseBoneIndex(INDEX_NONE)) && !FMath::IsNearlyZero(IKFootRootOrientationAlpha, KINDA_SMALL_NUMBER);

	// Rotate IK Foot Root
	if (bUpdateIKFootRoot)
	{
		const FQuat BoneRotation = FQuat(RotationAxisVector, ActualOrientationAngleRad * IKFootRootOrientationAlpha);

		FTransform IKFootRootTransform(Output.Pose.GetComponentSpaceTransform(IKFootData.IKFootRootBoneIndex));
		IKFootRootTransform.SetRotation(BoneRotation * IKFootRootTransform.GetRotation());
		IKFootRootTransform.NormalizeRotation();
		Output.Pose.SetComponentSpaceTransform(IKFootData.IKFootRootBoneIndex, IKFootRootTransform);

		// IK Feet 
		// These match the root orientation, so don't rotate them. Just preserve root rotation. 
		// We need to update their translation though, since we rotated their parent (the IK Foot Root bone).
		const int32 NumIKFootBones = IKFootData.IKFootBoneIndexArray.Num();
		const bool bUpdateIKFootBones = bUpdateIKFootRoot && (NumIKFootBones > 0);

		if (bUpdateIKFootBones)
		{
			const FQuat IKFootRotation = FQuat(RotationAxisVector, -ActualOrientationAngleRad * IKFootRootOrientationAlpha);

			for (int32 ArrayIndex = 0; ArrayIndex < NumIKFootBones; ArrayIndex++)
			{
				const FCompactPoseBoneIndex& IKFootBoneIndex = IKFootData.IKFootBoneIndexArray[ArrayIndex];

				FTransform IKFootBoneTransform(Output.Pose.GetComponentSpaceTransform(IKFootBoneIndex));
				IKFootBoneTransform.SetRotation(IKFootRotation * IKFootBoneTransform.GetRotation());
				IKFootBoneTransform.NormalizeRotation();
				Output.Pose.SetComponentSpaceTransform(IKFootBoneIndex, IKFootBoneTransform);
			}
		}
	}

	OutBoneTransforms.Sort(FCompareBoneTransformIndex());
	bIsFirstUpdate = false;
}

bool FAnimNode_OrientationWarping::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones)
{
#if ENABLE_ANIM_DEBUG
	if (AnimNodeOrientationWarpingEnable == 0)
	{
		return false;
	}
#endif
	if (RotationAxis == EAxis::None)
	{
		return false;
	}

	if (Mode == EWarpingEvaluationMode::Manual && UE::Anim::IsInvalidWarpingAngleDegrees(OrientationAngle, KINDA_SMALL_NUMBER))
	{
		return false;
	}

	if (SpineBoneDataArray.IsEmpty())
	{
		return false;
	}
	else
	{
		for (const auto& Spine : SpineBoneDataArray)
		{
			if (Spine.BoneIndex == INDEX_NONE)
			{
				return false;
			}
		}
	}

	if (IKFootData.IKFootRootBoneIndex == INDEX_NONE)
	{
		return false;
	}

	if (IKFootData.IKFootBoneIndexArray.IsEmpty())
	{
		return false;
	}
	else
	{
		for (const auto& IKFootBoneIndex : IKFootData.IKFootBoneIndexArray)
		{
			if (IKFootBoneIndex == INDEX_NONE)
			{
				return false;
			}
		}
	}
	return true;
}

void FAnimNode_OrientationWarping::InitializeBoneReferences(const FBoneContainer& RequiredBones)
{
	IKFootRootBone.Initialize(RequiredBones);
	IKFootData.IKFootRootBoneIndex = IKFootRootBone.GetCompactPoseIndex(RequiredBones);

	IKFootData.IKFootBoneIndexArray.Reset();
	for (auto& Bone : IKFootBones)
	{
		Bone.Initialize(RequiredBones);
		IKFootData.IKFootBoneIndexArray.Add(Bone.GetCompactPoseIndex(RequiredBones));
	}

	SpineBoneDataArray.Reset();
	for (auto& Bone : SpineBones)
	{
		Bone.Initialize(RequiredBones);
		SpineBoneDataArray.Add(FOrientationWarpingSpineBoneData(Bone.GetCompactPoseIndex(RequiredBones)));
	}

	if (SpineBoneDataArray.Num() > 0)
	{
		// Sort bones indices so we can transform parent before child
		SpineBoneDataArray.Sort(FOrientationWarpingSpineBoneData::FCompareBoneIndex());

		// Assign Weights.
		TArray<int32, TInlineAllocator<20>> IndicesToUpdate;

		for (int32 Index = SpineBoneDataArray.Num() - 1; Index >= 0; Index--)
		{
			// If this bone's weight hasn't been updated, scan its parents.
			// If parents have weight, we add it to 'ExistingWeight'.
			// split (1.f - 'ExistingWeight') between all members of the chain that have no weight yet.
			if (SpineBoneDataArray[Index].Weight == 0.f)
			{
				IndicesToUpdate.Reset(SpineBoneDataArray.Num());
				float ExistingWeight = 0.f;
				IndicesToUpdate.Add(Index);

				const FCompactPoseBoneIndex CompactBoneIndex = SpineBoneDataArray[Index].BoneIndex;
				for (int32 ParentIndex = Index - 1; ParentIndex >= 0; ParentIndex--)
				{
					if (RequiredBones.BoneIsChildOf(CompactBoneIndex, SpineBoneDataArray[ParentIndex].BoneIndex))
					{
						if (SpineBoneDataArray[ParentIndex].Weight > 0.f)
						{
							ExistingWeight += SpineBoneDataArray[ParentIndex].Weight;
						}
						else
						{
							IndicesToUpdate.Add(ParentIndex);
						}
					}
				}

				check(IndicesToUpdate.Num() > 0);
				const float WeightToShare = 1.f - ExistingWeight;
				const float IndividualWeight = WeightToShare / float(IndicesToUpdate.Num());

				for (int32 UpdateListIndex = 0; UpdateListIndex < IndicesToUpdate.Num(); UpdateListIndex++)
				{
					SpineBoneDataArray[IndicesToUpdate[UpdateListIndex]].Weight = IndividualWeight;
				}
			}
		}
	}
}

void FAnimNode_OrientationWarping::Reset(const FAnimationBaseContext& Context)
{
	bIsFirstUpdate = true;
	RootMotionDeltaDirection = FVector::ZeroVector;
	ManualRootMotionVelocity = FVector::ZeroVector;
	ActualOrientationAngleRad = 0.f;
}
