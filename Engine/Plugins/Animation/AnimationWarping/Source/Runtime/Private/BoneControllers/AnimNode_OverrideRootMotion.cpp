// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_OverrideRootMotion.h"

#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimNodeFunctionRef.h"
#include "Animation/AnimRootMotionProvider.h"
#include "HAL/IConsoleManager.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_OverrideRootMotion)

DECLARE_CYCLE_STAT(TEXT("OverrideRootMotion Eval"), STAT_OverrideRootMotion_Eval, STATGROUP_Anim);

void FAnimNode_OverrideRootMotion::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	Super::Initialize_AnyThread(Context);
	AnimInstanceProxy = Context.AnimInstanceProxy;
	Source.Initialize(Context);
}

void FAnimNode_OverrideRootMotion::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	Super::CacheBones_AnyThread(Context);
	Source.CacheBones(Context);
}

void FAnimNode_OverrideRootMotion::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)

	// Folded variables.
	FVector StaticOverrideVelocity = GetOverrideVelocity();
	float StaticAlpha = GetAlpha();
	
	FString DebugLine = DebugData.GetNodeName(this);
	DebugLine += FString::Printf(TEXT("(Override Velocity: %s, Alpha: %.3f"),
		*StaticOverrideVelocity.ToString(),
		StaticAlpha
		);
	
	Source.GatherDebugData(DebugData);
}

void FAnimNode_OverrideRootMotion::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	Super::Update_AnyThread(Context);

	GetEvaluateGraphExposedInputs().Execute(Context);

	DeltaTime = Context.GetDeltaTime();

	Source.Update(Context);
}

void FAnimNode_OverrideRootMotion::Evaluate_AnyThread(FPoseContext& Output)
{
	SCOPE_CYCLE_COUNTER(STAT_OverrideRootMotion_Eval);

	Super::Evaluate_AnyThread(Output);
	Source.Evaluate(Output);

	const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get();

	FTransform RootMotionTransform(FTransform::Identity);
	RootMotionProvider->ExtractRootMotion(Output.CustomAttributes, RootMotionTransform);

	FVector Velocity = GetOverrideVelocity();
	FVector Delta = Velocity * DeltaTime;
	FTransform Transform(Output.AnimInstanceProxy->GetComponentTransform().InverseTransformVector(Delta));

	RootMotionTransform.BlendWith(Transform, GetAlpha());
	
	RootMotionProvider->OverrideRootMotion(RootMotionTransform, Output.CustomAttributes);
}

const FVector& FAnimNode_OverrideRootMotion::GetOverrideVelocity() const
{
	return GET_ANIM_NODE_DATA(FVector, OverrideVelocity);
}

float FAnimNode_OverrideRootMotion::GetAlpha() const
{
	return GET_ANIM_NODE_DATA(float, Alpha);
}

