// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_SlopeWarping.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimNodeFunctionRef.h"
#include "Components/SkeletalMeshComponent.h"
#include "GameFramework/Character.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "Components/CapsuleComponent.h"
#include "Engine/SkeletalMeshSocket.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_SlopeWarping)

#if ENABLE_ANIM_DEBUG
TAutoConsoleVariable<int32> CVarAnimNodeSlopeWarpingDebug(TEXT("a.AnimNode.SlopeWarping.Debug"), 0, TEXT("Turn on debug for AnimNode_SlopeWarping"));
#endif

TAutoConsoleVariable<int32> CVarSlopeWarpingEnable(TEXT("a.AnimNode.SlopeWarping.Enable"), 1, TEXT("Toggle Slope Warping"));

DECLARE_CYCLE_STAT(TEXT("Slope Warping Eval"), STAT_SlopeWarping_Eval, STATGROUP_Anim);

FAnimNode_SlopeWarping::FAnimNode_SlopeWarping()
	: MySkelMeshComponent(nullptr)
	, MyMovementComponent(nullptr)
	, MyAnimInstanceProxy(nullptr)
	, GravityDir(-FVector::UpVector)
	, CustomFloorOffset(FVector::ZeroVector)
	, CachedDeltaTime(0.f)
	, TargetFloorNormalWorldSpace(ForceInitToZero)
	, TargetFloorOffsetLocalSpace(ForceInitToZero)
	, MaxStepHeight(50.f)
	, bKeepMeshInsideOfCapsule(true)
	, bPullPelvisDown(true)
	, bUseCustomFloorOffset(false)
	, bWasOnGround(false)
	, bShowDebug(false)
	, bFloorSmoothingInitialized(false)
	, ActorLocation(ForceInitToZero)
	, GravityDirCompSpace(ForceInitToZero)
{
	// Defaults
	PelvisOffsetInterpolator.SetDefaultSpringConstants(250.f);
	FloorNormalInterpolator.SetDefaultSpringConstants(1000.f);
	FloorOffsetInterpolator.SetDefaultSpringConstants(1000.f);
}

void FAnimNode_SlopeWarping::GatherDebugData(FNodeDebugData& DebugData)
{
	FString DebugLine = DebugData.GetNodeName(this);

	// 	DebugLine += "(";
	// 	AddDebugNodeData(DebugLine);
	// 	DebugLine += FString::Printf(TEXT(" Target: %s)"), *BoneToModify.BoneName.ToString());

	DebugData.AddDebugItem(DebugLine);

	ComponentPose.GatherDebugData(DebugData);
}

void FAnimNode_SlopeWarping::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	FAnimNode_SkeletalControlBase::Initialize_AnyThread(Context);

	MyAnimInstanceProxy = Context.AnimInstanceProxy;
	MySkelMeshComponent = Context.AnimInstanceProxy ? Context.AnimInstanceProxy->GetSkelMeshComponent() : nullptr;
	ACharacter* CharacterOwner = MySkelMeshComponent ? Cast<ACharacter>(MySkelMeshComponent->GetOwner()) : nullptr;
	MyMovementComponent = CharacterOwner ? CharacterOwner->GetCharacterMovement() : nullptr;

	FloorNormalInterpolator.SetPosition(-GravityDir);
}

void FAnimNode_SlopeWarping::UpdateInternal(const FAnimationUpdateContext& Context)
{
	FAnimNode_SkeletalControlBase::UpdateInternal(Context);

	CachedDeltaTime += Context.GetDeltaTime();
}

static FVector GetFootIKCentroid(const TArray<FSlopeWarpingFootData>& FeetData)
{
	check(FeetData.Num() > 0);

	FVector Centroid = FVector::ZeroVector;
	for (auto& FootData : FeetData)
	{
		Centroid += FootData.IKFootTransform.GetLocation();
	}

	Centroid /= float(FeetData.Num());
	return Centroid;
}

static FVector ReOrientNormal(const FVector& InNormal, FVector& PointA, const FVector& PointB)
{
	const FVector AxisX = (PointA - PointB).GetSafeNormal();
	if (!AxisX.IsNearlyZero() && !InNormal.IsNearlyZero() && (FMath::Abs(AxisX | InNormal) > DELTA))
	{
		const FVector AxisY = AxisX ^ InNormal;
		const FVector AxisZ = AxisX ^ AxisY;

		// Make sure our normal points upwards. (take into account gravity dir?)
		return ((AxisZ | FVector::UpVector) > 0.f) ? AxisZ : -AxisZ;
	}
	else
	{
		return InNormal;
	}
}

FVector FAnimNode_SlopeWarping::GetFloorNormalFromMovementComponent() const
{
	check(MyMovementComponent);
	return (MyMovementComponent->CurrentFloor.bBlockingHit) ? MyMovementComponent->CurrentFloor.HitResult.ImpactNormal : -GravityDir;
}

void FAnimNode_SlopeWarping::SetWorldSpaceTargetFloorInfoFromMovementComponent()
{
	TargetFloorNormalWorldSpace = GetFloorNormalFromMovementComponent();

	check(MySkelMeshComponent);
	check(MyMovementComponent);

	FVector WorldSpaceFloorOffset(FVector::ZeroVector);

	// Take into account offsets created by capsule.
	if (ACharacter* CharacterOwner = Cast<ACharacter>(MySkelMeshComponent->GetOwner()))
	{
		UCapsuleComponent* Capsule = Cast<UCapsuleComponent>(CharacterOwner->GetRootComponent());
		if (Capsule)
		{
			// Take into account MovementComponent floor dist offset
			const FVector MovementComponentFloorDistOffset = MyMovementComponent->CurrentFloor.bBlockingHit ? MyMovementComponent->CurrentFloor.GetDistanceToFloor() * GravityDir : FVector::ZeroVector;
			WorldSpaceFloorOffset += MovementComponentFloorDistOffset;

			// Take into account offset between mesh and base of capsule
			FVector MeshToCapsuleBaseOffset = FVector::ZeroVector;
			if (MyMovementComponent->CurrentFloor.bBlockingHit)
			{
				const FVector CapsuleBaseLoc = Capsule->GetComponentLocation() + GravityDir * Capsule->GetScaledCapsuleHalfHeight();
				const FVector MeshLoc = MySkelMeshComponent->GetComponentLocation();
				const FVector ProjectedMeshLoc = (FMath::Abs(GravityDir | TargetFloorNormalWorldSpace) > DELTA) ? FMath::LinePlaneIntersection(MeshLoc, MeshLoc + GravityDir, CapsuleBaseLoc, TargetFloorNormalWorldSpace) : MeshLoc;

				MeshToCapsuleBaseOffset = (MeshLoc - ProjectedMeshLoc);
				WorldSpaceFloorOffset += MeshToCapsuleBaseOffset;
			}

			// Take into account Capsule radius against slope.
			const float CapsuleRadius = Capsule->GetScaledCapsuleRadius();
			const FVector CapsuleRadiusAgainstSlopeOffset = GravityDir * CapsuleRadius * (1.f - (TargetFloorNormalWorldSpace | -GravityDir));
			WorldSpaceFloorOffset += CapsuleRadiusAgainstSlopeOffset;

#if ENABLE_ANIM_DEBUG
			if (bShowDebug)
			{
				FString DebugString = FString::Printf(TEXT("DistToFloorOffset: %s, MeshToCapsuleBaseOffset: %s, CapsuleRadiusAgainstSlopeOffset: %s, WorldSpaceFloorOffset: %s"),
					*MovementComponentFloorDistOffset.ToCompactString(), *MeshToCapsuleBaseOffset.ToCompactString(), *CapsuleRadiusAgainstSlopeOffset.ToCompactString(), *WorldSpaceFloorOffset.ToCompactString());
				MyAnimInstanceProxy->AnimDrawDebugOnScreenMessage(DebugString, FColor::Red);
			}
#endif
		}
	}

	// Turn floor offset to local space.
	TargetFloorOffsetLocalSpace = MySkelMeshComponent->GetComponentTransform().InverseTransformVectorNoScale(WorldSpaceFloorOffset);
}

static bool HasCharacterTeleported(const FVector& InPreviousLoc, const FVector& CurrentLoc, const UCharacterMovementComponent& InCharacterMovementComponent)
{
	return (CurrentLoc - InPreviousLoc).SizeSquared2D() > FMath::Square(InCharacterMovementComponent.NetworkNoSmoothUpdateDistance);
}

void FAnimNode_SlopeWarping::GetSmoothedFloorInfo(FCSPose<FCompactPose>& MeshBases, FVector& OutFloorNormal, FVector& OutFloorOffset)
{
	check(MyMovementComponent);
	check(MySkelMeshComponent);

	AActor* OwnerActor = MySkelMeshComponent->GetOwner();
	// Use mesh component location instead to help simulated proxies where capsule simulation is disabled, and it's mesh location is interpolated instead.
	const FVector NewActorLocation = MySkelMeshComponent->GetComponentLocation();

	// Detect when Actor teleports to reset our interpolation
	const bool bJustTeleported = !bFloorSmoothingInitialized || HasCharacterTeleported(ActorLocation, NewActorLocation, *MyMovementComponent);
	const bool bIsOnGround = ((MyMovementComponent->MovementMode == MOVE_Walking) || (MyMovementComponent->MovementMode == MOVE_NavWalking)) && MyMovementComponent->CurrentFloor.bBlockingHit;

	const FVector ActorLocationLastFrame = ActorLocation;
	ActorLocation = NewActorLocation;

	// if bFloorSmoothingInitialized is false, then bJustTeleported should be true
	check(bFloorSmoothingInitialized || (bFloorSmoothingInitialized != bJustTeleported));
	if (bJustTeleported)
	{
		bFloorSmoothingInitialized = true;

		TargetFloorOffsetLocalSpace = FVector::ZeroVector;
		FloorOffsetInterpolator.Reset(TargetFloorOffsetLocalSpace);

		TargetFloorNormalWorldSpace = -GravityDir;
		FloorNormalInterpolator.Reset(TargetFloorNormalWorldSpace);
	}
	else
	{
		if (bIsOnGround)
		{
			// Compensate for sudden capsule moves
			if (bWasOnGround)
			{
				const FVector CapsuleFloorNormal = GetFloorNormalFromMovementComponent();
				const FVector AdjustedActorLocationLastFrame = (FMath::Abs(GravityDir | CapsuleFloorNormal) > DELTA) ? FMath::LinePlaneIntersection(ActorLocation, ActorLocation + GravityDir, ActorLocationLastFrame, CapsuleFloorNormal) : ActorLocation;

				// Delta Capsule
				const FVector CapsuleMoveOffsetWorldSpace = (ActorLocation - AdjustedActorLocationLastFrame);
				if (!CapsuleMoveOffsetWorldSpace.IsNearlyZero(KINDA_SMALL_NUMBER))
				{
					const FVector CapsuleMoveOffsetLocalSpace = MySkelMeshComponent->GetComponentTransform().InverseTransformVectorNoScale(CapsuleMoveOffsetWorldSpace);
					FloorOffsetInterpolator.OffsetPosition(-CapsuleMoveOffsetLocalSpace);
					TargetFloorOffsetLocalSpace -= CapsuleMoveOffsetLocalSpace;

#if ENABLE_ANIM_DEBUG
					if (bShowDebug)
					{
						FString DebugString = FString::Printf(TEXT("CapsuleMoveOffset: %s"), *CapsuleMoveOffsetLocalSpace.ToCompactString());
						MyAnimInstanceProxy->AnimDrawDebugOnScreenMessage(DebugString, FColor::Red);
					}
#endif
				}
			}

			SetWorldSpaceTargetFloorInfoFromMovementComponent();
		}
		else
		{
#if ENABLE_ANIM_DEBUG
			if (bShowDebug)
			{
				FString DebugString = FString::Printf(TEXT("SlopeWarping NOT ON GROUND"));
				MyAnimInstanceProxy->AnimDrawDebugOnScreenMessage(DebugString, FColor::Red);
			}
#endif
			TargetFloorOffsetLocalSpace = FVector::ZeroVector;
			TargetFloorNormalWorldSpace = -GravityDir;
		}
	}

	bWasOnGround = bIsOnGround;

	// Interpolate floor normal in world space.
	if (!FloorNormalInterpolator.IsPositionEqualTo(TargetFloorNormalWorldSpace))
	{
		const FVector SmoothedFloorNormalWorldSpace = FloorNormalInterpolator.Update(TargetFloorNormalWorldSpace, CachedDeltaTime).GetSafeNormal();
		FloorNormalInterpolator.SetPosition(SmoothedFloorNormalWorldSpace);
	}
	OutFloorNormal = MySkelMeshComponent->GetComponentTransform().InverseTransformVectorNoScale(FloorNormalInterpolator.GetPosition());

	if (bUseCustomFloorOffset)
	{
		OutFloorOffset = CustomFloorOffset;
		FloorOffsetInterpolator.SetPosition(CustomFloorOffset);
	}
	else
	{
		// Interpolate floor offset in local space.
		const FVector MaxStep = -GravityDir * MaxStepHeight;
		TargetFloorOffsetLocalSpace = ClampVector(TargetFloorOffsetLocalSpace, -MaxStep, MaxStep);
		OutFloorOffset = FloorOffsetInterpolator.Update(TargetFloorOffsetLocalSpace, CachedDeltaTime);

#if ENABLE_ANIM_DEBUG
		if (bShowDebug)
		{
			FString DebugString = FString::Printf(TEXT("OutFloorOffset: %s, TargetFloorOffsetLocalSpace: %s")
				, *OutFloorOffset.ToCompactString(), *TargetFloorOffsetLocalSpace.ToCompactString());
			MyAnimInstanceProxy->AnimDrawDebugOnScreenMessage(DebugString, FColor::Red);
		}
#endif
	}
}

void FAnimNode_SlopeWarping::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	SCOPE_CYCLE_COUNTER(STAT_SlopeWarping_Eval);

	check(OutBoneTransforms.Num() == 0);

	const FBoneContainer& RequiredBones = Output.Pose.GetPose().GetBoneContainer();

#if ENABLE_ANIM_DEBUG
	bShowDebug = (CVarAnimNodeSlopeWarpingDebug.GetValueOnAnyThread() == 1);
#endif

	const FCompactPoseBoneIndex IKFootRootBoneIndex = IKFootRootBone.GetCompactPoseIndex(RequiredBones);
	FTransform IKFootRootTransform = Output.Pose.GetComponentSpaceTransform(IKFootRootBoneIndex);
	if (!ensureMsgf(IKFootRootTransform.ContainsNaN() == false, TEXT("FAnimNode_SlopeWarping, Incoming IKFootRootTransform contains NaN (%s)"), *IKFootRootTransform.ToString()))
	{
		IKFootRootTransform = FTransform::Identity;
	}

	const FVector IKFootRootLoc = IKFootRootTransform.GetLocation();
	const FVector IKFootFloorNormal = IKFootRootTransform.GetUnitAxis(EAxis::Z);

	if (!ensureMsgf(GravityDir.ContainsNaN() == false, TEXT("FAnimNode_SlopeWarping, Incoming GravityDir contains NaN (%s)"), *GravityDir.ToCompactString()))
	{
		GravityDir = -FVector::UpVector;
	}
	GravityDirCompSpace = MyAnimInstanceProxy->GetComponentTransform().InverseTransformVectorNoScale(GravityDir).GetSafeNormal();

	// Get Feet IK Transforms
	{
		for (auto& FootData : FeetData)
		{
			FootData.IKFootTransform = Output.Pose.GetComponentSpaceTransform(FootData.IKFootBoneIndex);
			if (!ensureMsgf(FootData.IKFootTransform.ContainsNaN() == false, TEXT("FAnimNode_SlopeWarping, Incoming IKFootTransform contains NaN (%s)"), *FootData.IKFootTransform.ToString()))
			{
				FootData.IKFootTransform = FTransform::Identity;
			}

			const FVector FootIKLoc = FootData.IKFootTransform.GetLocation();

#if ENABLE_ANIM_DEBUG
			if (bShowDebug)
			{
				const FVector FootIKWorldLoc = MyAnimInstanceProxy->GetComponentTransform().TransformPosition(FootIKLoc);
				MyAnimInstanceProxy->AnimDrawDebugSphere(FootIKWorldLoc, 5.f, 16, FColor::Red);
			}
#endif
		}
	}

	// Initial IK Feet Centroid
	const FVector InitialCentroidLoc = GetFootIKCentroid(FeetData);

#if ENABLE_ANIM_DEBUG
	if (bShowDebug)
	{
		FVector CentroidWorldLoc = MyAnimInstanceProxy->GetComponentTransform().TransformPosition(InitialCentroidLoc);
		MyAnimInstanceProxy->AnimDrawDebugSphere(CentroidWorldLoc, 5.f, 4, FColor::Red);
	}
#endif

	FVector FloorNormal, FloorOffset;
	GetSmoothedFloorInfo(Output.Pose, FloorNormal, FloorOffset);

	if (!ensureMsgf(FloorNormal.ContainsNaN() == false, TEXT("FAnimNode_SlopeWarping, Incoming FloorNormal contains NaN (%s)"), *FloorNormal.ToCompactString()))
	{
		FloorNormal = FVector::UpVector;
	}

	if (!ensureMsgf(FloorOffset.ContainsNaN() == false, TEXT("FAnimNode_SlopeWarping, Incoming FloorOffset contains NaN (%s)"), *FloorOffset.ToCompactString()))
	{
		FloorOffset = FVector::ZeroVector;
	}

	const bool bHasFloorOffset = !FloorOffset.IsNearlyZero(KINDA_SMALL_NUMBER);

#if ENABLE_ANIM_DEBUG
	if (bShowDebug)
	{
		const FVector FloorOriginWorldSpace = MyAnimInstanceProxy->GetComponentTransform().TransformPosition(FloorOffset);
		const FVector FloorNormalWorldSpace = MyAnimInstanceProxy->GetComponentTransform().TransformVectorNoScale(FloorNormal);
		MyAnimInstanceProxy->AnimDrawDebugSphere(FloorOriginWorldSpace, 3.f, 8, FColor::Yellow);
		MyAnimInstanceProxy->AnimDrawDebugDirectionalArrow(FloorOriginWorldSpace, FloorOriginWorldSpace + FloorNormalWorldSpace * 100.f, 10.f, FColor::Yellow);

		FString DebugString = FString::Printf(TEXT("FloorOffset: %s"), *FloorOffset.ToCompactString());
		MyAnimInstanceProxy->AnimDrawDebugOnScreenMessage(DebugString, FColor::Red);
	}
#endif

	// Orient along floor normal

	// Find Delta Rotation take takes us from Animated Slope to In-Game Slope
	FQuat DeltaSlopeRotation = FQuat::FindBetweenNormals(IKFootFloorNormal, FloorNormal);
	if (!ensureMsgf(DeltaSlopeRotation.ContainsNaN() == false, TEXT("FAnimNode_SlopeWarping, Incoming DeltaSlopeRotation contains NaN (%s)"), *DeltaSlopeRotation.ToString()))
	{
		DeltaSlopeRotation = FQuat::Identity;
	}

	// Rotate our Joint quaternion by this delta rotation
	const bool bPoseNeedsToBeRotated = !DeltaSlopeRotation.IsIdentity();
	if (bPoseNeedsToBeRotated)
	{
		IKFootRootTransform.SetRotation(DeltaSlopeRotation * IKFootRootTransform.GetRotation());
		IKFootRootTransform.NormalizeRotation();
	}
	IKFootRootTransform.AddToTranslation(FloorOffset);

	// Warp Feet IK to floor
	if (bPoseNeedsToBeRotated)
	{
		for (auto& FootData : FeetData)
		{
			// Warp Location
			const FVector WarpedFootIKLoc = DeltaSlopeRotation.RotateVector(FootData.IKFootTransform.GetLocation());
			FootData.IKFootTransform.SetTranslation(WarpedFootIKLoc + FloorOffset);

			// Warp Rotation
			FootData.IKFootTransform.SetRotation(DeltaSlopeRotation * FootData.IKFootTransform.GetRotation());
			FootData.IKFootTransform.NormalizeRotation();

#if ENABLE_ANIM_DEBUG
			if (bShowDebug)
			{
				const FVector FootIKWorldLoc = MyAnimInstanceProxy->GetComponentTransform().TransformPosition(FootData.IKFootTransform.GetLocation());
				MyAnimInstanceProxy->AnimDrawDebugSphere(FootIKWorldLoc, 5.f, 16, FColor::Green);
			}
#endif
		}
	}
	else if (bHasFloorOffset)
	{
		for (auto& FootData : FeetData)
		{
			FootData.IKFootTransform.AddToTranslation(FloorOffset);
		}
	}

	// Pelvis Adjustment
	const FCompactPoseBoneIndex PelvisBoneIndex = PelvisBone.GetCompactPoseIndex(RequiredBones);
	FTransform PelvisTransform = Output.Pose.GetComponentSpaceTransform(PelvisBoneIndex);
	if (!ensureMsgf(PelvisTransform.ContainsNaN() == false, TEXT("FAnimNode_SlopeWarping, Incoming PelvisTransform contains NaN (%s)"), *PelvisTransform.ToString()))
	{
		PelvisTransform = FTransform::Identity;
	}

	const FVector InitialPelvisLocation = PelvisTransform.GetLocation();

#if ENABLE_ANIM_DEBUG
	if (bShowDebug)
	{
		const FVector PelvisWorldLoc = MyAnimInstanceProxy->GetComponentTransform().TransformPosition(PelvisTransform.GetLocation());
		MyAnimInstanceProxy->AnimDrawDebugSphere(PelvisWorldLoc, 5.f, 16, FColor::Red);
	}
#endif

	PelvisTransform.AddToTranslation(FloorOffset);
	const FVector PelvisLocationBeforeInterp = PelvisTransform.GetLocation();

	// keep pelvis inline with feet centroid.
	if (bPoseNeedsToBeRotated)
	{
		const FVector PostRotationFeetCentroid = GetFootIKCentroid(FeetData);
		const FVector CentroidOffset = (PostRotationFeetCentroid - InitialCentroidLoc) - FloorOffset;

		PelvisTransform.AddToTranslation(CentroidOffset);

#if ENABLE_ANIM_DEBUG
		if (bShowDebug)
		{
			MyAnimInstanceProxy->AnimDrawDebugSphere(MyAnimInstanceProxy->GetComponentTransform().TransformPosition(PostRotationFeetCentroid), 5.f, 4, FColor::Orange);
			MyAnimInstanceProxy->AnimDrawDebugSphere(MyAnimInstanceProxy->GetComponentTransform().TransformPosition(PelvisTransform.GetLocation()), 5.f, 16, FColor::Orange);
		}
#endif
	}

	// pull hips down to prevent leg over-extension
	if (bPoseNeedsToBeRotated && bPullPelvisDown)
	{
		// Test current pelvis offset and modify if needed
		FVector PelvisOffset = PelvisTransform.GetLocation() - InitialPelvisLocation;

		bool bCheckForLegOverextension = true;
		int32 NumIterations = 0;
		while (bCheckForLegOverextension && (NumIterations++ < 4))
		{
			bCheckForLegOverextension = false;
			for (auto& FootData : FeetData)
			{
				const FVector HipLocation = Output.Pose.GetComponentSpaceTransform(FootData.HipBoneIndex).GetLocation();
				const FVector FKFootLocation = Output.Pose.GetComponentSpaceTransform(FootData.FKFootBoneIndex).GetLocation();

				const float OriginalLegLength = (HipLocation - FKFootLocation).Size();
				const FVector IKFootToAdjustedHip = HipLocation + PelvisOffset - FootData.IKFootTransform.GetLocation();
				const float DistanceToIKTarget = IKFootToAdjustedHip.Size();

				// Pull down hips if we have to
				if (DistanceToIKTarget > OriginalLegLength)
				{
					FVector NewHipLocation = FootData.IKFootTransform.GetLocation() + IKFootToAdjustedHip.GetSafeNormal() * OriginalLegLength;

					PelvisOffset = NewHipLocation - HipLocation;

// 					PelvisLocAfterOffset = FootData.IKFootTransform.GetLocation() + (PelvisLocAfterOffset - FootData.IKFootTransform.GetLocation()).GetSafeNormal() * OriginalLegLength;

					// modified hips, take another iteration to make sure this hasn't made other feet unreachable.
					bCheckForLegOverextension = true;
				}
			}
		}

		// Set our position to the modified position if it has changed
		PelvisTransform.SetLocation(InitialPelvisLocation + PelvisOffset);

#if ENABLE_ANIM_DEBUG
		if (bShowDebug)
		{
			const FVector PelvisWorldLoc = MyAnimInstanceProxy->GetComponentTransform().TransformPosition(PelvisTransform.GetLocation());
			MyAnimInstanceProxy->AnimDrawDebugSphere(PelvisWorldLoc, 5.f, 16, FColor::Emerald);
		}
#endif
	}

	// Adjustment to keep pose inside of capsule
	if (bKeepMeshInsideOfCapsule)
	{
		// Project Pelvis transform onto slope.
		FVector PelvisOffset = PelvisTransform.GetLocation() - PelvisLocationBeforeInterp;
		if (!PelvisOffset.IsNearlyZero())
		{
			FVector ProjectedPelvisOffset = FVector::VectorPlaneProject(PelvisOffset, FloorNormal);

			// Shift everything along slope to keep mesh inside of capsule.
			PelvisTransform.AddToTranslation(-ProjectedPelvisOffset);

			for (FSlopeWarpingFootData& FootData : FeetData)
			{
				FootData.IKFootTransform.AddToTranslation(-ProjectedPelvisOffset);
			}

			IKFootRootTransform.AddToTranslation(-ProjectedPelvisOffset);
		}
	}

	// Interpolate Pelvis offset smoothly
	if (true)
	{
		FVector PelvisOffset = PelvisTransform.GetLocation() - PelvisLocationBeforeInterp;
		PelvisOffsetInterpolator.Update(PelvisOffset, CachedDeltaTime);
		PelvisTransform.SetLocation(PelvisLocationBeforeInterp + PelvisOffsetInterpolator.GetPosition());

#if ENABLE_ANIM_DEBUG
		if (bShowDebug)
		{
			const FVector PelvisWorldLoc = MyAnimInstanceProxy->GetComponentTransform().TransformPosition(PelvisTransform.GetLocation());
			MyAnimInstanceProxy->AnimDrawDebugSphere(PelvisWorldLoc, 5.f, 16, FColor::Purple);
		}
#endif
	}

	const bool bPelvisInterpolated = !PelvisOffsetInterpolator.GetPosition().IsNearlyZero(KINDA_SMALL_NUMBER);

	// rotate hips towards IK feet target (to help IK nodes), and pull legs in towards pelvis to prevent over extension
	const bool bNeedsToAdjustHips = (bPelvisInterpolated || bPoseNeedsToBeRotated);
	if (bNeedsToAdjustHips)
	{
		const FVector PelvisOffset = (PelvisTransform.GetLocation() - InitialPelvisLocation);
		for (auto& FootData : FeetData)
		{
			const FTransform HipTransform = Output.Pose.GetComponentSpaceTransform(FootData.HipBoneIndex);
			const FTransform FKFootTransform = Output.Pose.GetComponentSpaceTransform(FootData.FKFootBoneIndex);
			FTransform AdjustedHipTransform = HipTransform;

			AdjustedHipTransform.AddToTranslation(PelvisOffset);

			const FVector InitialDir = (FKFootTransform.GetLocation() - HipTransform.GetLocation()).GetSafeNormal();
			const FVector TargetDir = (FootData.IKFootTransform.GetLocation() - AdjustedHipTransform.GetLocation()).GetSafeNormal();

#if ENABLE_ANIM_DEBUG
			if (bShowDebug)
			{
				const FTransform SkelTM = MyAnimInstanceProxy->GetComponentTransform();
				MyAnimInstanceProxy->AnimDrawDebugLine(SkelTM.TransformPosition(HipTransform.GetLocation()), SkelTM.TransformPosition(FKFootTransform.GetLocation()), FColor::Red);
				MyAnimInstanceProxy->AnimDrawDebugLine(SkelTM.TransformPosition(AdjustedHipTransform.GetLocation()), SkelTM.TransformPosition(FootData.IKFootTransform.GetLocation()), FColor::Green);
			}
#endif

			if (!InitialDir.IsNearlyZero() && !TargetDir.IsNearlyZero())
			{
				// Find Delta Rotation take takes us from Old to New dir
				const FQuat DeltaRotation = FQuat::FindBetweenNormals(InitialDir, TargetDir);

				// Rotate our Joint quaternion by this delta rotation
				AdjustedHipTransform.SetRotation(DeltaRotation * AdjustedHipTransform.GetRotation());
				AdjustedHipTransform.NormalizeRotation();

				// Add adjusted hip transform
				if (ensureMsgf(AdjustedHipTransform.ContainsNaN() == false, TEXT("FAnimNode_SlopeWarping, Outgoing AdjustedHipTransform contains NaN (%s)"), *AdjustedHipTransform.ToString()))
				{
					OutBoneTransforms.Add(FBoneTransform(FootData.HipBoneIndex, AdjustedHipTransform));
				}

				// Clamp IK Feet bone based on FK leg. To prevent over-extension and preserve animated pose.
				{
					const float FKLength = FVector::Dist(FKFootTransform.GetLocation(), HipTransform.GetLocation());
					const float IKLength = FVector::Dist(FootData.IKFootTransform.GetLocation(), AdjustedHipTransform.GetLocation());
					if (IKLength > FKLength)
					{
						FootData.IKFootTransform.SetLocation(AdjustedHipTransform.GetLocation() + TargetDir * FKLength);
					}
				}
			}
		}
	}

	// Add transforms that have been modified
	{
		// Add adjusted IK Foot Root
		if (bPoseNeedsToBeRotated)
		{
			if (ensureMsgf(IKFootRootTransform.ContainsNaN() == false, TEXT("FAnimNode_SlopeWarping, Outgoing IKFootRootTransform contains NaN (%s)"), *IKFootRootTransform.ToString()))
			{
				OutBoneTransforms.Add(FBoneTransform(IKFootRootBoneIndex, IKFootRootTransform));
			}
		}

		if (bHasFloorOffset)
		{
			const FCompactPoseBoneIndex RootBoneIndex(0);
			FTransform RootBoneTransform(Output.Pose.GetComponentSpaceTransform(RootBoneIndex));
			RootBoneTransform.AddToTranslation(FloorOffset);

			if (ensureMsgf(RootBoneTransform.ContainsNaN() == false, TEXT("FAnimNode_SlopeWarping, Outgoing RootBoneTransform contains NaN (%s)"), *RootBoneTransform.ToString()))
			{
				OutBoneTransforms.Add(FBoneTransform(RootBoneIndex, RootBoneTransform));
			}
		}

		if (bPoseNeedsToBeRotated || bPelvisInterpolated)
		{
			if (ensureMsgf(PelvisTransform.ContainsNaN() == false, TEXT("FAnimNode_SlopeWarping, Outgoing PelvisTransform contains NaN (%s)"), *PelvisTransform.ToString()))
			{
				OutBoneTransforms.Add(FBoneTransform(PelvisBoneIndex, PelvisTransform));
			}
		}

#if ENABLE_ANIM_DEBUG
		if (bShowDebug)
		{
			FVector PelvisWorldLoc = MyAnimInstanceProxy->GetComponentTransform().TransformPosition(PelvisTransform.GetLocation());
			MyAnimInstanceProxy->AnimDrawDebugSphere(PelvisWorldLoc, 5.f, 16, FColor::Blue);
		}
#endif

		if (bPoseNeedsToBeRotated)
		{
			for (auto& FootData : FeetData)
			{
				if (ensureMsgf(FootData.IKFootTransform.ContainsNaN() == false, TEXT("FAnimNode_SlopeWarping, Outgoing FootData.IKFootTransform contains NaN (%s)"), *FootData.IKFootTransform.ToString()))
				{
					OutBoneTransforms.Add(FBoneTransform(FootData.IKFootBoneIndex, FootData.IKFootTransform));
				}

#if ENABLE_ANIM_DEBUG
				if (bShowDebug)
				{
					FVector IKFootWorldLocation = MyAnimInstanceProxy->GetComponentTransform().TransformPosition(FootData.IKFootTransform.GetLocation());
					MyAnimInstanceProxy->AnimDrawDebugSphere(IKFootWorldLocation, 5.f, 16, FColor::Blue);
				}
#endif
			}
		}

#if ENABLE_ANIM_DEBUG
		if (bShowDebug)
		{
			FVector FinalCentroidLoc = GetFootIKCentroid(FeetData);
			FVector CentroidWorldLoc = MyAnimInstanceProxy->GetComponentTransform().TransformPosition(FinalCentroidLoc);
			MyAnimInstanceProxy->AnimDrawDebugSphere(CentroidWorldLoc, 5.f, 4, FColor::Blue);
		}
#endif

	}

	// Sort OutBoneTransforms so indices are in increasing order.
	OutBoneTransforms.Sort(FCompareBoneTransformIndex());

	// Clear time accumulator, to be filled during next update.
	CachedDeltaTime = 0.f;
}

bool FAnimNode_SlopeWarping::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones)
{
	const bool bIsEnabled = (CVarSlopeWarpingEnable.GetValueOnAnyThread() == 1);
	return MyMovementComponent && bIsEnabled && (IKFootRootBone.GetCompactPoseIndex(RequiredBones) != INDEX_NONE) && (PelvisBone.GetCompactPoseIndex(RequiredBones) != INDEX_NONE) && (FeetData.Num() > 0);
}

static FCompactPoseBoneIndex FindHipBoneIndex(const FCompactPoseBoneIndex& InFootBoneIndex, const int32& NumBonesInLimb, const FBoneContainer& RequiredBones)
{
	FCompactPoseBoneIndex BoneIndex = InFootBoneIndex;
	if (BoneIndex != INDEX_NONE)
	{
		FCompactPoseBoneIndex ParentBoneIndex = RequiredBones.GetParentBoneIndex(BoneIndex);

		int32 NumIterations = NumBonesInLimb;
		while ((NumIterations-- > 0) && (ParentBoneIndex != INDEX_NONE))
		{
			BoneIndex = ParentBoneIndex;
			ParentBoneIndex = RequiredBones.GetParentBoneIndex(BoneIndex);
		};
	}

	return BoneIndex;
}

/** OutBoneIndex and OutLocalTransform will only be set if found! */
static void SetBoneIndexAndLocalTransformFromSocket(const FName& InSocketName, const USkeletalMeshComponent& SkelMeshComp, const FBoneContainer& RequiredBones, FCompactPoseBoneIndex& OutBoneIndex, FTransform& OutLocalTransform)
{
	if (InSocketName != NAME_None)
	{
		const USkeletalMeshSocket* SkelMeshSocket = SkelMeshComp.GetSocketByName(InSocketName);
		if (SkelMeshSocket)
		{
			OutBoneIndex = RequiredBones.MakeCompactPoseIndex(FMeshPoseBoneIndex(RequiredBones.GetPoseBoneIndexForBoneName(SkelMeshSocket->BoneName)));
			OutLocalTransform = SkelMeshSocket->GetSocketLocalTransform();
		}
	}
}

void FAnimNode_SlopeWarping::InitializeBoneReferences(const FBoneContainer& RequiredBones)
{
	IKFootRootBone.Initialize(RequiredBones);

	PelvisBone.Initialize(RequiredBones);

	FeetData.Reset();
	check(MySkelMeshComponent);
	for (auto& FootDef : FeetDefinitions)
	{
		FootDef.IKFootBone.Initialize(RequiredBones);
		FootDef.FKFootBone.Initialize(RequiredBones);

		FSlopeWarpingFootData FootData;
		FootData.IKFootBoneIndex = FootDef.IKFootBone.GetCompactPoseIndex(RequiredBones);
		FootData.FKFootBoneIndex = FootDef.FKFootBone.GetCompactPoseIndex(RequiredBones);
		FootData.HipBoneIndex = FindHipBoneIndex(FootData.FKFootBoneIndex, FMath::Max(FootDef.NumBonesInLimb, 1), RequiredBones);

		if ((FootData.IKFootBoneIndex != INDEX_NONE)
			&& (FootData.FKFootBoneIndex != INDEX_NONE)
			&& (FootData.HipBoneIndex != INDEX_NONE))
		{
			FootData.FootDefinitionPtr = &FootDef;
			FeetData.Add(FootData);
		}
	}
}
