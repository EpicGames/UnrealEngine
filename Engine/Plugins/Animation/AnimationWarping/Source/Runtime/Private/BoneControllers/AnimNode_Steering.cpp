// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_Steering.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimNodeFunctionRef.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Animation/AnimSequenceHelpers.h"
#include "Math/SpringMath.h"
#include "HAL/IConsoleManager.h"
#include "Animation/AnimTrace.h"
#include "BoneControllers/AnimNode_OffsetRootBone.h"
#include "Logging/LogVerbosity.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_Steering)

bool bAnimNodeSteeringEnabled = true;
static FAutoConsoleVariableRef CVarAnimNodeSteeringEnabled(
	TEXT("a.AnimNode.Steering.Enabled"),
	bAnimNodeSteeringEnabled,
	TEXT("True will enable steering anim nodes. Equivalent to setting alpha to non-zero.")
);

void FAnimNode_Steering::UpdateInternal(const FAnimationUpdateContext& Context)
{
	FAnimNode_SkeletalControlBase::UpdateInternal(Context);

	if (UE::AnimationWarping::FRootOffsetProvider* RootOffsetProvider = Context.GetMessage<UE::AnimationWarping::FRootOffsetProvider>())
	{
		RootBoneTransform = RootOffsetProvider->GetRootTransform();
	}
	else
	{
		RootBoneTransform = Context.AnimInstanceProxy->GetComponentTransform();
	}
}

void FAnimNode_Steering::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	FAnimNode_SkeletalControlBase::Initialize_AnyThread(Context);
	AngularVelocity = FVector::ZeroVector;
}

void FAnimNode_Steering::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)

	FString DebugLine = DebugData.GetNodeName(this);
	// Just track alpha, there are clearer tools for the visualization of steering elsewhere.  
	DebugLine += FString::Printf(TEXT("Alpha:%.3f"),
		Alpha
		);
	
	ComponentPose.GatherDebugData(DebugData);
}

void FAnimNode_Steering::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	const float DeltaSeconds = Output.AnimInstanceProxy->GetDeltaSeconds();

	if (DeltaSeconds > 0.f)
	{
		if (Alpha > 0.0f && bAnimNodeSteeringEnabled)
		{
			const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get();
			ensureMsgf(RootMotionProvider, TEXT("Steering expected a valid root motion delta provider interface."));

			if (RootMotionProvider)
			{
				FTransform ThisFrameRootMotionTransform = FTransform::Identity;
				if (RootMotionProvider->ExtractRootMotion(Output.CustomAttributes, ThisFrameRootMotionTransform))
				{
					float CurrentSpeed = ThisFrameRootMotionTransform.GetTranslation().Length() / DeltaSeconds;
					if (CurrentSpeed > DisableSteeringBelowSpeed)
					{
						FQuat RootBoneRotation = RootBoneTransform.GetRotation();

						UE_VLOG_ARROW(Output.AnimInstanceProxy->GetAnimInstanceObject(), "Steering", Display,
							RootBoneTransform.GetLocation(),
							RootBoneTransform.GetLocation()  + RootBoneRotation.GetRightVector() * 90,
							FColor::Green, TEXT(""));
						
						UE_VLOG_ARROW(Output.AnimInstanceProxy->GetAnimInstanceObject(), "Steering", Display,
							RootBoneTransform.GetLocation(),
							RootBoneTransform.GetLocation()  + TargetOrientation.GetRightVector() * 100,
							FColor::Blue, TEXT(""));

						FQuat DeltaToTargetOrientation =  RootBoneRotation.Inverse() * TargetOrientation;

						if (AnimatedTargetTime > 0)
						{
							if (UAnimSequenceBase* AnimSequence = Cast<UAnimSequenceBase>(CurrentAnimAsset))
							{
								FTransform PredictedRootMotionTransform;
								if (bMirrored && MirrorDataTable)
								{
									PredictedRootMotionTransform = UE::Anim::ExtractRootMotionFromAnimationAsset(AnimSequence, MirrorDataTable, CurrentAnimAssetTime, AnimatedTargetTime, AnimSequence->bLoop);
								}
								else
								{
									const FAnimExtractContext Context(static_cast<double>(CurrentAnimAssetTime), true, FDeltaTimeRecord(AnimatedTargetTime), AnimSequence->bLoop);
									PredictedRootMotionTransform = AnimSequence->ExtractRootMotion(Context);
								}

								FQuat PredictedRootMotionQuat = PredictedRootMotionTransform.GetRotation();
								FRotator PredictedRootMotionRot = FRotator(PredictedRootMotionQuat);
								float PredictedRootMotionYaw = PredictedRootMotionRot.Yaw;
							
								if (fabs(PredictedRootMotionYaw) > RootMotionThreshold)
								{
									UE_VLOG_ARROW(Output.AnimInstanceProxy->GetAnimInstanceObject(), "Steering", Display,
										RootBoneTransform.GetLocation(),
										RootBoneTransform.GetLocation()  + (PredictedRootMotionQuat * RootBoneRotation).GetRightVector() * 100,
										FColor::Orange, TEXT(""));

									float YawToTargetOrientation = FRotator(DeltaToTargetOrientation).Yaw;

									// pick the rotation direction that is the shortest path from the endpoint of the current animated rotation
									if (PredictedRootMotionYaw - YawToTargetOrientation > 180)
									{
										YawToTargetOrientation += 360;
									}
									else if (YawToTargetOrientation - PredictedRootMotionYaw > 180)
									{
										YawToTargetOrientation -=360;
									}

									float Ratio =  YawToTargetOrientation / PredictedRootMotionYaw ;
									Ratio = FMath::Clamp(Ratio, MinScaleRatio, MaxScaleRatio);
									
									// Account for alpha
									Ratio = FMath::Lerp(1.0f, Ratio, Alpha);

									FRotator ThisFrameRootMotionRotator(ThisFrameRootMotionTransform.GetRotation());
									ThisFrameRootMotionRotator.Yaw *= Ratio;
									ThisFrameRootMotionTransform.SetRotation(FQuat(ThisFrameRootMotionRotator));

									// Account for future scaling in linear error correction
									PredictedRootMotionRot.Yaw *= Ratio;
									PredictedRootMotionQuat = PredictedRootMotionRot.Quaternion();

									DeltaToTargetOrientation = PredictedRootMotionQuat.Inverse() * RootBoneRotation.Inverse() * TargetOrientation;
								}
							}
						}
						
						if (CurrentSpeed > DisableAdditiveBelowSpeed)
						{
							// Apply linear correction
							FQuat LinearCorrection = FQuat::Identity;
							SpringMath::CriticalSpringDamperQuat(IN OUT LinearCorrection, IN OUT AngularVelocity, DeltaToTargetOrientation, SpringMath::HalfLifeToSmoothingTime(ProceduralTargetTime), DeltaSeconds);

							UE_VLOG_ARROW(Output.AnimInstanceProxy->GetAnimInstanceObject(), "Steering", Display,
								RootBoneTransform.GetLocation(),
								RootBoneTransform.GetLocation() + (RootBoneTransform.GetRotation() * LinearCorrection).GetRightVector() * 120,
								FColor::Magenta, TEXT(""));

							FQuat ThisFrameRotation = ThisFrameRootMotionTransform.GetRotation() * LinearCorrection;

							UE_VLOG_ARROW(Output.AnimInstanceProxy->GetAnimInstanceObject(), "Steering", Display,
								RootBoneTransform.GetLocation(),
								RootBoneTransform.GetLocation() + (RootBoneTransform.GetRotation() * ThisFrameRotation).GetRightVector() * 140,
								FColor::Red, TEXT(""));

							ThisFrameRootMotionTransform.SetRotation(FQuat::Slerp(ThisFrameRootMotionTransform.GetRotation(), ThisFrameRotation, Alpha));
						}

						RootMotionProvider->OverrideRootMotion(ThisFrameRootMotionTransform, Output.CustomAttributes);
					}
				}
			}
		}
	}
}
