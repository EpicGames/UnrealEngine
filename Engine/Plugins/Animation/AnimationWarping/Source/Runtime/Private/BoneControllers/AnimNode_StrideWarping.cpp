// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_StrideWarping.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimNodeFunctionRef.h"
#include "Animation/AnimRootMotionProvider.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_StrideWarping)

DECLARE_CYCLE_STAT(TEXT("StrideWarping Eval"), STAT_StrideWarping_Eval, STATGROUP_Anim);

#if ENABLE_ANIM_DEBUG
static TAutoConsoleVariable<int32> CVarAnimNodeStrideWarpingDebug(TEXT("a.AnimNode.StrideWarping.Debug"), 0, TEXT("Turn on visualization debugging for Stride Warping"));
static TAutoConsoleVariable<int32> CVarAnimNodeStrideWarpingVerbose(TEXT("a.AnimNode.StrideWarping.Verbose"), 0, TEXT("Turn on verbose graph debugging for Stride Warping"));
static TAutoConsoleVariable<int32> CVarAnimNodeStrideWarpingEnable(TEXT("a.AnimNode.StrideWarping.Enable"), 1, TEXT("Toggle Stride Warping"));
#endif

void FAnimNode_StrideWarping::GatherDebugData(FNodeDebugData& DebugData)
{
	FString DebugLine = DebugData.GetNodeName(this);
#if ENABLE_ANIM_DEBUG
	if (CVarAnimNodeStrideWarpingVerbose.GetValueOnAnyThread() == 1)
	{ 
		DebugLine += FString::Printf(TEXT("\n - Evaluation Mode: (%s)"), Mode == EWarpingEvaluationMode::Graph ? TEXT("Graph") : TEXT("Manual"));
		DebugLine += FString::Printf(TEXT("\n - Stride Scale: (%.3fd)"), ActualStrideScale);
		DebugLine += FString::Printf(TEXT("\n - Stride Direction: (%s)"), *(ActualStrideDirection.ToCompactString()));
		if (Mode == EWarpingEvaluationMode::Graph)
		{
			DebugLine += FString::Printf(TEXT("\n - Locomotion Speed: (%.3fd)"), LocomotionSpeed);
#if WITH_EDITORONLY_DATA
			DebugLine += FString::Printf(TEXT("\n - Root Motion Delta Attribute Found: %s)"), (bFoundRootMotionAttribute) ? TEXT("true") : TEXT("false"));
			DebugLine += FString::Printf(TEXT("\n - Root Motion Direction: (%s)"), *(CachedRootMotionDeltaTranslation.GetSafeNormal().ToCompactString()));
			DebugLine += FString::Printf(TEXT("\n - Root Motion Speed: (%.3fd)"), CachedRootMotionDeltaSpeed);
#endif
			DebugLine += FString::Printf(TEXT("\n - Min Root Motion Speed Threshold: (%.3fd)"), MinRootMotionSpeedThreshold);
		}
		DebugLine += FString::Printf(TEXT("\n - Floor Normal: (%s)"), *(FloorNormalDirection.Value.ToCompactString()));
		DebugLine += FString::Printf(TEXT("\n - Gravity Direction: (%s)"), *(GravityDirection.Value.ToCompactString()));
	}
	else
#endif
	{
		DebugLine += FString::Printf(TEXT("(Stride Scale: %.3fd, Direction: %s)"), ActualStrideScale, *(ActualStrideDirection.ToCompactString()));
	}
	DebugData.AddDebugItem(DebugLine);
	ComponentPose.GatherDebugData(DebugData);
}

void FAnimNode_StrideWarping::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	FAnimNode_SkeletalControlBase::Initialize_AnyThread(Context);
	
	AnimInstanceProxy = Context.AnimInstanceProxy;
	StrideScaleModifierState.Reinitialize();
	ActualStrideDirection = FVector::ForwardVector;
	ActualStrideScale = 1.f;
#if WITH_EDITORONLY_DATA
	CachedRootMotionDeltaTranslation = FVector::ZeroVector;
	CachedRootMotionDeltaSpeed = 0.f;
#endif
}

void FAnimNode_StrideWarping::UpdateInternal(const FAnimationUpdateContext& Context)
{
	FAnimNode_SkeletalControlBase::UpdateInternal(Context);
	CachedDeltaTime = Context.GetDeltaTime();
}

void FAnimNode_StrideWarping::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	SCOPE_CYCLE_COUNTER(STAT_StrideWarping_Eval);
	check(OutBoneTransforms.IsEmpty());

	const FVector PreviousStrideDirection = ActualStrideDirection;
	ActualStrideDirection = StrideDirection;
	ActualStrideScale = StrideScale;

	bool bGraphDrivenWarping = false;
	const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get();

	if (Mode == EWarpingEvaluationMode::Graph)
	{
		bGraphDrivenWarping = !!RootMotionProvider;
		if (!bGraphDrivenWarping)
		{
			UE_LOGF(LogAnimation, Error, "FAnimNode_StrideWarping, Graph-driven mode expected a valid root motion delta provider interface");
			return;
		}
	}
	
	if (!bGraphDrivenWarping)
	{
		// If we aren't using graph driven warping, we need to validate the input variables are valid
		if (StrideDirection.ContainsNaN())
		{
			UE_LOGF(LogAnimation, Error, "Invalid stride direction in FAnimNode_StrideWarping");
			return;
		}

		if (!FMath::IsFinite(StrideScale))
		{
			UE_LOGF(LogAnimation, Error, "Invalid stride scale in FAnimNode_StrideWarping");
			return;
		}
	}

	FTransform RootMotionTransformDelta = FTransform::Identity;
#if WITH_EDITORONLY_DATA
	bFoundRootMotionAttribute = false;
#else
	FVector CachedRootMotionDeltaTranslation = FVector::ZeroVector;
#endif
	if (bGraphDrivenWarping)
	{
		// Graph driven stride warping will override the manual stride direction with the intent of the current animation sub-graph's accumulated root motion
		bGraphDrivenWarping = RootMotionProvider->ExtractRootMotion(Output.CustomAttributes, RootMotionTransformDelta);
		if (bGraphDrivenWarping)
		{
			CachedRootMotionDeltaTranslation = RootMotionTransformDelta.GetTranslation();
			// If there's no root motion delta, keep the previous stride direction
			ActualStrideDirection = CachedRootMotionDeltaTranslation.GetSafeNormal(UE_SMALL_NUMBER, PreviousStrideDirection);
#if WITH_EDITORONLY_DATA
			// Graph driven Stride Warping expects a root motion delta to be present in the attribute stream.
			bFoundRootMotionAttribute = true;
#endif
		}
		// Early exit on missing root motion delta attribute
		else if (bDisableIfMissingRootMotion)
		{
			return;
		}
	}

	const FBoneContainer& RequiredBones = Output.Pose.GetPose().GetBoneContainer();
	const FTransform IKFootRootTransform = Output.Pose.GetComponentSpaceTransform(IKFootRootBone.GetCompactPoseIndex(RequiredBones));
	FVector ResolvedFloorNormal = FloorNormalDirection.AsComponentSpaceDirection(AnimInstanceProxy, IKFootRootTransform);
	if (ResolvedFloorNormal.ContainsNaN())
	{
		UE_LOGF(LogAnimation, Error, "FAnimNode_StrideWarping, Resolved FloorNormal contains NaN (%ls); falling back to UpVector", *ResolvedFloorNormal.ToCompactString());
		ResolvedFloorNormal = FVector::UpVector;
	}
	FVector ResolvedGravityDirection = GravityDirection.AsComponentSpaceDirection(AnimInstanceProxy, IKFootRootTransform);
	if (ResolvedGravityDirection.ContainsNaN())
	{
		UE_LOGF(LogAnimation, Error, "FAnimNode_StrideWarping, Resolved GravityDirection contains NaN (%ls); falling back to DownVector", *ResolvedGravityDirection.ToCompactString());
		ResolvedGravityDirection = FVector::DownVector;
	}

	if (bOrientStrideDirectionUsingFloorNormal)
	{
		const FVector StrideWarpingAxis = ResolvedFloorNormal ^ ActualStrideDirection;
		ActualStrideDirection = StrideWarpingAxis ^ ResolvedFloorNormal;
	}

#if ENABLE_ANIM_DEBUG
	bool bDebugging = false;
#if WITH_EDITORONLY_DATA
	bDebugging = bDebugging || bEnableDebugDraw;
#else
	constexpr float DebugDrawScale = 1.f;
#endif
	bDebugging = bDebugging || CVarAnimNodeStrideWarpingDebug.GetValueOnAnyThread() == 1;
	if (bDebugging)
	{
		// Draw Floor Normal
		AnimInstanceProxy->AnimDrawDebugDirectionalArrow(
			AnimInstanceProxy->GetComponentTransform().TransformPosition(IKFootRootTransform.GetLocation()),
			AnimInstanceProxy->GetComponentTransform().TransformPosition(IKFootRootTransform.GetLocation() + ResolvedFloorNormal * 25.f * DebugDrawScale),
			40.f * DebugDrawScale, FColor::Blue, false, 0.f, 2.f * DebugDrawScale);
	}
#endif

	// Get all foot IK Transforms
	for (auto& Foot : FootData)
	{
		Foot.IKFootBoneTransform = Output.Pose.GetComponentSpaceTransform(Foot.IKFootBoneIndex);
#if ENABLE_ANIM_DEBUG
		if (bDebugging
#if WITH_EDITORONLY_DATA
			&& bDebugDrawIKFootOrigin
#endif
		)
		{
			const FVector FootWorldLocation = AnimInstanceProxy->GetComponentTransform().TransformPosition(Foot.IKFootBoneTransform.GetLocation());
			AnimInstanceProxy->AnimDrawDebugSphere(FootWorldLocation, 8.f * DebugDrawScale, 16, FColor::Red);
		}
#endif
	}

	if (bGraphDrivenWarping)
	{
#if WITH_EDITORONLY_DATA
		CachedRootMotionDeltaSpeed = !FMath::IsNearlyZero(CachedDeltaTime) ? (CachedRootMotionDeltaTranslation.Size() / CachedDeltaTime) : 0.f;
#else
		const float CachedRootMotionDeltaSpeed = !FMath::IsNearlyZero(CachedDeltaTime) ? (CachedRootMotionDeltaTranslation.Size() / CachedDeltaTime) : 0.f;
#endif

		if (CachedRootMotionDeltaSpeed <= MinRootMotionSpeedThreshold || FMath::IsNearlyZero(CachedRootMotionDeltaSpeed))
		{
			// If root motion speed is under the threshold, snap back to no stride adjustment.
			// If interpolation is on, it will blend the result
			ActualStrideScale = 1.0f;
		}
		else
		{
			// Graph driven stride scale factor will be determined by the ratio of the
			// locomotion (capsule/physics) speed against the animation root motion speed
			ActualStrideScale = LocomotionSpeed / CachedRootMotionDeltaSpeed;
		}
	}

	// Allow the opportunity for stride scale clamping and interpolation regardless of evaluation mode
	ActualStrideScale = StrideScaleModifierState.ApplyTo(StrideScaleModifier, ActualStrideScale, CachedDeltaTime);
	if (!FMath::IsFinite(ActualStrideScale))
	{
		UE_LOGF(LogAnimation, Error, "FAnimNode_StrideWarping, ActualStrideScale is non-finite (%f); falling back to 1.0", ActualStrideScale);
		ActualStrideScale = 1.0f;
	}

	if (bGraphDrivenWarping)
	{
		// Forward the side effects of stride warping on the root motion contribution for this sub-graph
		RootMotionTransformDelta.ScaleTranslation(ActualStrideScale);
		const bool bRootMotionOverridden = RootMotionProvider->OverrideRootMotion(RootMotionTransformDelta, Output.CustomAttributes);
		if (!bRootMotionOverridden)
		{
			UE_LOGF(LogAnimation, Error, "FAnimNode_StrideWarping, Graph-driven mode expected a root motion delta to be present in the attribute stream prior to warping/overriding it");
			return;
		}
	}

	// Scale IK feet bones along Stride Warping Axis, from the Thigh bone location.
	for (auto& Foot : FootData)
	{
		// Stride Warping along Stride Warping Axis
		const FVector IKFootLocation = Foot.IKFootBoneTransform.GetLocation();
		const FVector ThighBoneLocation = Output.Pose.GetComponentSpaceTransform(Foot.ThighBoneIndex).GetLocation();

		// Project Thigh Bone Location on plane made of FootIKLocation and FloorPlaneNormal, along Gravity Dir.
		// This will be the StrideWarpingPlaneOrigin
		const FVector StrideWarpingPlaneOrigin = (FMath::Abs(ResolvedGravityDirection | ResolvedFloorNormal) > DELTA) ? FMath::LinePlaneIntersection(ThighBoneLocation, ThighBoneLocation + ResolvedGravityDirection, IKFootLocation, ResolvedFloorNormal) : IKFootLocation;

		// Project FK Foot along StrideWarping Plane, this will be our Scale Origin
		const FVector ScaleOrigin = FVector::PointPlaneProject(IKFootLocation, StrideWarpingPlaneOrigin, ActualStrideDirection);

		// Now the ScaleOrigin and IKFootLocation are forming a line parallel to the floor, and we can scale the IK foot.
		const FVector WarpedLocation = ScaleOrigin + (IKFootLocation - ScaleOrigin) * ActualStrideScale;
		check(!WarpedLocation.ContainsNaN());
		Foot.IKFootBoneTransform.SetLocation(WarpedLocation);

#if ENABLE_ANIM_DEBUG
		if (bDebugging
#if WITH_EDITORONLY_DATA
			&& bDebugDrawIKFootAdjustment
#endif
		)
		{
			const FVector FootWorldLocation = AnimInstanceProxy->GetComponentTransform().TransformPosition(Foot.IKFootBoneTransform.GetLocation());
			AnimInstanceProxy->AnimDrawDebugSphere(FootWorldLocation, 8.f * DebugDrawScale, 16, FColor::Green);

			const FVector ScaleOriginWorldLoc = AnimInstanceProxy->GetComponentTransform().TransformPosition(ScaleOrigin);
			AnimInstanceProxy->AnimDrawDebugSphere(ScaleOriginWorldLoc, 8.f * DebugDrawScale, 16, FColor::Yellow);
		}
#endif
	}

	FVector PelvisOffset = FVector::ZeroVector;
	const FCompactPoseBoneIndex PelvisBoneIndex = PelvisBone.GetCompactPoseIndex(RequiredBones);

	FTransform PelvisTransform = Output.Pose.GetComponentSpaceTransform(PelvisBoneIndex);
	const FVector InitialPelvisLocation = PelvisTransform.GetLocation();

	TArray<float, TInlineAllocator<10>> FKFootDistancesToPelvis;
	FKFootDistancesToPelvis.Reserve(FootData.Num());

	TArray<FVector, TInlineAllocator<10>> IKFootLocations;
	IKFootLocations.Reserve(FootData.Num());
	
	for (auto& Foot : FootData)
	{
		const FVector FKFootLocation = Output.Pose.GetComponentSpaceTransform(Foot.FKFootBoneIndex).GetLocation();
		FKFootDistancesToPelvis.Add(FVector::Dist(FKFootLocation, InitialPelvisLocation));

		const FVector IKFootLocation = Foot.IKFootBoneTransform.GetLocation();
		IKFootLocations.Add(IKFootLocation);
	}

	// Adjust Pelvis down if needed to keep foot contact with the ground and prevent over-extension
	PelvisTransform = PelvisIKFootSolver.Solve(PelvisTransform, FKFootDistancesToPelvis, IKFootLocations, CachedDeltaTime);

#if ENABLE_ANIM_DEBUG
	if (bDebugging
#if WITH_EDITORONLY_DATA
		&& bDebugDrawPelvisAdjustment
#endif
	)
	{
		// Draw Adjustments in Pelvis location
		AnimInstanceProxy->AnimDrawDebugSphere(AnimInstanceProxy->GetComponentTransform().TransformPosition(InitialPelvisLocation), 8.f * DebugDrawScale, 16, FColor::Red);
		AnimInstanceProxy->AnimDrawDebugSphere(AnimInstanceProxy->GetComponentTransform().TransformPosition(PelvisTransform.GetLocation()), 8.f * DebugDrawScale, 16, FColor::Blue);
	}

	if (bDebugging)
	{
		// Draw Stride Direction
		AnimInstanceProxy->AnimDrawDebugDirectionalArrow(
			AnimInstanceProxy->GetComponentTransform().TransformPosition(InitialPelvisLocation),
			AnimInstanceProxy->GetComponentTransform().TransformPosition(InitialPelvisLocation + ActualStrideDirection * ActualStrideScale * 100.f * DebugDrawScale),
			40.f * DebugDrawScale, FColor::Red, false, 0.f, 2.f * DebugDrawScale);

		// Draw text on mesh in world space
		{
			TStringBuilder<1024> DebugLine;

			DebugLine.Appendf(TEXT("\n - Stride Scale: (%.3fd)"), ActualStrideScale);
			DebugLine.Appendf(TEXT("\n - Stride Direction: (%s)"), *(ActualStrideDirection.ToCompactString()));
			
			if (Mode == EWarpingEvaluationMode::Graph)
			{
				DebugLine.Appendf(TEXT("\n - Locomotion Speed: (%.3fd)"), LocomotionSpeed);

#if WITH_EDITORONLY_DATA
				if (bFoundRootMotionAttribute)
				{
					DebugLine.Appendf(TEXT("\n - Root Motion: Direction(%s), Speed(%.3fd)"), *(CachedRootMotionDeltaTranslation.GetSafeNormal().ToCompactString()), CachedRootMotionDeltaSpeed);
				}
				else
				{
					DebugLine.Appendf(TEXT("\n - Root Motion: NO ATTRIBUTE FOUND"), *(ActualStrideDirection.ToCompactString()));
				}
#endif

				DebugLine.Appendf(TEXT("\n - Min Root Motion Speed Threshold: (%.3fd)"), MinRootMotionSpeedThreshold);
			}

			AnimInstanceProxy->AnimDrawDebugInWorldMessage(DebugLine.ToString(), FVector::UpVector * 50.0f, FColor::Yellow, 1.f /*TextScale*/);
		}
	}
#endif
	// Add adjusted pelvis transform
	check(!PelvisTransform.ContainsNaN());
	OutBoneTransforms.Add(FBoneTransform(PelvisBoneIndex, PelvisTransform));

	// Compute final offset to use below
	PelvisOffset = (PelvisTransform.GetLocation() - InitialPelvisLocation);

	// Rotate Thigh bones to help IK, and maintain leg shape.
	if (bCompensateIKUsingFKThighRotation)
	{
		for (auto& Foot : FootData)
		{
			const FTransform ThighTransform = Output.Pose.GetComponentSpaceTransform(Foot.ThighBoneIndex);
			const FTransform FKFootTransform = Output.Pose.GetComponentSpaceTransform(Foot.FKFootBoneIndex);
			
			FTransform AdjustedThighTransform = ThighTransform;
			AdjustedThighTransform.AddToTranslation(PelvisOffset);

			const FVector InitialDir = (FKFootTransform.GetLocation() - ThighTransform.GetLocation()).GetSafeNormal();
			const FVector TargetDir = (Foot.IKFootBoneTransform.GetLocation() - AdjustedThighTransform.GetLocation()).GetSafeNormal();
			
#if ENABLE_ANIM_DEBUG
			if (bDebugging
#if WITH_EDITORONLY_DATA
				&& bDebugDrawThighAdjustment
#endif
			)
			{
				AnimInstanceProxy->AnimDrawDebugLine(
					AnimInstanceProxy->GetComponentTransform().TransformPosition(ThighTransform.GetLocation()),
					AnimInstanceProxy->GetComponentTransform().TransformPosition(FKFootTransform.GetLocation()),
					FColor::Red, false, 0.f, 2.f * DebugDrawScale);

				AnimInstanceProxy->AnimDrawDebugLine(
					AnimInstanceProxy->GetComponentTransform().TransformPosition(AdjustedThighTransform.GetLocation()),
					AnimInstanceProxy->GetComponentTransform().TransformPosition(Foot.IKFootBoneTransform.GetLocation()),
					FColor::Green, false, 0.f, 2.f * DebugDrawScale);
			}
#endif
			// Find Delta Rotation take takes us from Old to New dir
			const FQuat DeltaRotation = FQuat::FindBetweenNormals(InitialDir, TargetDir);

			// Rotate our Joint quaternion by this delta rotation
			AdjustedThighTransform.SetRotation(DeltaRotation * AdjustedThighTransform.GetRotation());

			// Add adjusted thigh transform
			check(!AdjustedThighTransform.ContainsNaN());
			OutBoneTransforms.Add(FBoneTransform(Foot.ThighBoneIndex, AdjustedThighTransform));

			// Clamp IK Feet bone based on FK leg. To prevent over-extension and preserve animated motion.
			if (bClampIKUsingFKLimits)
			{
				const float FKLength = FVector::Dist(FKFootTransform.GetLocation(), ThighTransform.GetLocation());
				const float IKLength = FVector::Dist(Foot.IKFootBoneTransform.GetLocation(), AdjustedThighTransform.GetLocation());
				if (IKLength > FKLength)
				{
					const FVector ClampedFootLocation = AdjustedThighTransform.GetLocation() + TargetDir * FKLength;
					Foot.IKFootBoneTransform.SetLocation(ClampedFootLocation);
				}
			}
		}
	}

	// Add final IK feet transforms
	for (auto& Foot : FootData)
	{
#if ENABLE_ANIM_DEBUG
		if (bDebugging
#if WITH_EDITORONLY_DATA
			&& bDebugDrawIKFootFinal
#endif
		)
		{
			AnimInstanceProxy->AnimDrawDebugSphere(AnimInstanceProxy->GetComponentTransform().TransformPosition(Foot.IKFootBoneTransform.GetLocation()), 8.f * DebugDrawScale, 16, FColor::Blue);
		}
#endif
		OutBoneTransforms.Add(FBoneTransform(Foot.IKFootBoneIndex, Foot.IKFootBoneTransform));
	}

	// Sort OutBoneTransforms so indices are in increasing order.
	OutBoneTransforms.Sort(FCompareBoneTransformIndex());
}

bool FAnimNode_StrideWarping::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones)
{
#if ENABLE_ANIM_DEBUG
	if (CVarAnimNodeStrideWarpingEnable.GetValueOnAnyThread() == 0)
	{
		return false;
	}
#endif
	if (!PelvisBone.IsValidToEvaluate(RequiredBones))
	{
		return false;
	}

	if (!IKFootRootBone.IsValidToEvaluate(RequiredBones))
	{
		return false;
	}

	if (FootData.IsEmpty())
	{
		return false;
	}
	else
	{
		for (const auto& Foot : FootData)
		{
			if (Foot.IKFootBoneIndex == INDEX_NONE || Foot.FKFootBoneIndex == INDEX_NONE || Foot.ThighBoneIndex == INDEX_NONE)
			{
				return false;
			}
		}
	}

	return true;
}

void FAnimNode_StrideWarping::InitializeBoneReferences(const FBoneContainer& RequiredBones)
{
	IKFootRootBone.Initialize(RequiredBones);
	PelvisBone.Initialize(RequiredBones);
	FootData.Empty();

	for (auto& Foot : FootDefinitions)
	{
		Foot.IKFootBone.Initialize(RequiredBones);
		Foot.FKFootBone.Initialize(RequiredBones);
		Foot.ThighBone.Initialize(RequiredBones);

		FStrideWarpingFootData StrideFootData;
		StrideFootData.IKFootBoneIndex = Foot.IKFootBone.GetCompactPoseIndex(RequiredBones);
		StrideFootData.FKFootBoneIndex = Foot.FKFootBone.GetCompactPoseIndex(RequiredBones);
		StrideFootData.ThighBoneIndex = Foot.ThighBone.GetCompactPoseIndex(RequiredBones);

		if ((StrideFootData.IKFootBoneIndex != INDEX_NONE) && (StrideFootData.FKFootBoneIndex != INDEX_NONE) && (StrideFootData.ThighBoneIndex != INDEX_NONE))
		{
			FootData.Add(StrideFootData);
		}
	}
}
