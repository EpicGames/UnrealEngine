// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneControllers/AnimNode_WarpTest.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimRootMotionProvider.h"
#include "BoneControllers/AnimNode_OffsetRootBone.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_WarpTest)

void FAnimNode_WarpTest::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	Super::Initialize_AnyThread(Context);
	Source.Initialize(Context);
}

void FAnimNode_WarpTest::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	Super::CacheBones_AnyThread(Context);
	Source.CacheBones(Context);
}

void FAnimNode_WarpTest::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	Super::Update_AnyThread(Context);
	GetEvaluateGraphExposedInputs().Execute(Context);
	Source.Update(Context);

	if (Transforms.IsEmpty())
	{
		CurrentTransformIndex = 0;
		CurrentTime = 0.f;
	}
	else
	{
		if (CurrentTransformIndex >= Transforms.Num())
		{
			CurrentTransformIndex = 0;
		}

		CurrentTime += Context.GetDeltaTime();
		if (CurrentTime > SecondsToWait)
		{
			CurrentTime -= SecondsToWait;
			CurrentTransformIndex = (CurrentTransformIndex + 1) % Transforms.Num();
		}
	}

	if (UE::AnimationWarping::FRootOffsetProvider* RootOffsetProvider = Context.GetMessage<UE::AnimationWarping::FRootOffsetProvider>())
	{
		ComponentTransform = RootOffsetProvider->GetRootTransform();
	}
	else
	{
		ComponentTransform = Context.AnimInstanceProxy->GetComponentTransform();
	}
}

void FAnimNode_WarpTest::Evaluate_AnyThread(FPoseContext& Output)
{
	Super::Evaluate_AnyThread(Output);
	Source.Evaluate(Output);
	
	if (!Transforms.IsEmpty())
	{
		const FTransform& WarpTo = Transforms[CurrentTransformIndex];
		const FTransform RootMotion = WarpTo.GetRelativeTransform(ComponentTransform);

		const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get();
		RootMotionProvider->OverrideRootMotion(RootMotion, Output.CustomAttributes);

#if ENABLE_VISUAL_LOG
		if (FVisualLogger::IsRecording())
		{
			UObject* AnimInstanceObject = Output.AnimInstanceProxy->GetAnimInstanceObject();

			static const TCHAR* LogName = TEXT("WarpTestNode");
			UE_VLOG_SEGMENT_THICK(AnimInstanceObject, LogName, Display, WarpTo.GetLocation(), WarpTo.GetRotation().GetAxisX() * 100.f + WarpTo.GetLocation(), FColor::Red, 1, TEXT(""));
			UE_VLOG_SEGMENT_THICK(AnimInstanceObject, LogName, Display, WarpTo.GetLocation(), WarpTo.GetRotation().GetAxisY() * 100.f + WarpTo.GetLocation(), FColor::Blue, 1, TEXT(""));

			UE_VLOG_SEGMENT_THICK(AnimInstanceObject, LogName, Display, ComponentTransform.GetLocation(), ComponentTransform.GetRotation().GetAxisX() * 80.f + ComponentTransform.GetLocation(), FColor::Black, 1, TEXT(""));
			UE_VLOG_SEGMENT_THICK(AnimInstanceObject, LogName, Display, ComponentTransform.GetLocation(), ComponentTransform.GetRotation().GetAxisY() * 80.f + ComponentTransform.GetLocation(), FColor::Green, 1, TEXT(""));
		}
#endif // ENABLE_VISUAL_LOG
	}
}
