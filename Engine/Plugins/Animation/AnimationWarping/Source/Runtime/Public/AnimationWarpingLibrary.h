// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimExecutionContext.h"
#include "Animation/AnimNodeReference.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "AnimationWarpingLibrary.generated.h"

#define UE_API ANIMATIONWARPINGRUNTIME_API

struct FAnimNode_OffsetRootBone;
class UCurveFloat;
class UCurveVector;

USTRUCT(BlueprintType)
struct FOffsetRootBoneAnimNodeReference : public FAnimNodeReference
{
	GENERATED_BODY()

	typedef FAnimNode_OffsetRootBone FInternalNodeType;
};

// Exposes operations related to Animation Warping
UCLASS(MinimalAPI, Experimental)
class UAnimationWarpingLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Get the current world space transform from the offset root bone animgraph node */
    UFUNCTION(BlueprintPure, Category = "Animation|BlendStack", meta = (BlueprintThreadSafe))
    static UE_API FTransform GetOffsetRootTransform(const FAnimNodeReference& Node);
	
	/** Helper function to extract the value of a curve in an animation at a given time */
	UFUNCTION(BlueprintPure, Category = "Animation", meta = (BlueprintThreadSafe))
	static UE_API bool GetCurveValueFromAnimation(const UAnimSequenceBase* Animation, FName CurveName, float Time, float& OutValue);

	/** Helper function to extract the float value from a curve asset at a given time. To be used inside AnimationBlueprint and assumes AnimBP holds reference to CurveAsset for it to be valid. */
	UFUNCTION(BlueprintPure, Category = "Animation", meta = (BlueprintThreadSafe))
	static bool GetFloatValueFromCurve(const UCurveFloat* InCurve, float Time, float& OutValue);

	/** Helper function to extract the vector value from a curve asset at a given time. To be used inside AnimationBlueprint and assumes AnimBP holds reference to CurveAsset for it to be valid. */
	UFUNCTION(BlueprintPure, Category = "Animation", meta = (BlueprintThreadSafe))
	static bool GetVectorValueFromCurve(const UCurveVector* InCurve, float Time, FVector& OutValue);

	/** Get a root bone offset node context from an anim node context */
	UFUNCTION(BlueprintPure, Category = "Animation|Offset Root Bone", meta = (BlueprintThreadSafe, DisplayName = "Convert to Offset Root Bone node"))
	static UE_API void ConvertToOffsetRootBoneNodePure(const FAnimNodeReference& Node, FOffsetRootBoneAnimNodeReference& OffsetRootBoneNode, bool& Result);

	/** Apply a delta translation and rotation to the simulated transform inside a offset root bone node. This is useful for moving the
	 * root bone offset, e.g. when standing on a moving platform */
	UFUNCTION(BlueprintCallable, Category = "Animation|Offset Root Bone", meta = (BlueprintThreadSafe))
	static void ApplyDeltaToOffsetRootBone(const FOffsetRootBoneAnimNodeReference& OffsetRootBoneNode, FVector TranslationWorld, FQuat RotationWorld);
};

#undef UE_API
