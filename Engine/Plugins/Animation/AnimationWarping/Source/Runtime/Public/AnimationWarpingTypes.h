// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimationWarpingTypes.generated.h"

UENUM(BlueprintType)
enum class EOffsetRootBoneMode : uint8
{
	// Accumulate the mesh component's movement into the offset.
	// In this mode, if the mesh component moves 
	// the offset will counter the motion, and the root will stay in place
	Accumulate,
	// Continuously interpolate the offset out
	// In this mode, if the mesh component moves
	// The root will stay behind, but will attempt to catch up
	Interpolate,
	// Stops accumulating the mesh component's movement delta into the root offset
	// In this mode, whatever offset we have will be locked but we will still consume animated root motion 
	LockOffsetAndConsumeAnimation,
	// Stops accumulating the mesh component's movement delta into the root offset
	// In this mode, whatever offset we have will be locked but we will still consume animated root motion, as long as it's decreasing the offset.
	LockOffsetIncreaseAndConsumeAnimation,
	// Stops accumulating the mesh component's movement delta into the root offset
	// In this mode, whatever offset we have will be locked and we will ignore animated root motion
	LockOffsetAndIgnoreAnimation,
	// Release the offset and stop accumulating the mesh component's movement delta.
	// In this mode we will "blend out" the offset
	Release,
};
