// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoneControllers/BoneControllerTypes.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "Kismet/KismetMathLibrary.h"
#include "Engine/EngineTypes.h"
#include "VisualLogger/VisualLogger.h"

#include "AnimNode_FootPlacement.generated.h"

#define UE_API ANIMATIONWARPINGRUNTIME_API

struct FAnimationInitializeContext;
struct FComponentSpacePoseContext;
struct FNodeDebugData;

#if ENABLE_ANIM_DEBUG && ENABLE_VISUAL_LOG
#define ENABLE_FOOTPLACEMENT_DEBUG 1
#else
#define  ENABLE_FOOTPLACEMENT_DEBUG 0
#endif

namespace UE::Anim::FootPlacement
{
	enum class EPlantType
	{
		Unplanted,
		Planted,
		Replanted
	};

	struct FLegRuntimeData
	{
		int32 Idx = -1;

		// Bone information that can be cached once per-lod change.
		struct FBoneData
		{
			FCompactPoseBoneIndex FKIndex = FCompactPoseBoneIndex(INDEX_NONE);
			FCompactPoseBoneIndex BallIndex = FCompactPoseBoneIndex(INDEX_NONE);
			FCompactPoseBoneIndex IKIndex = FCompactPoseBoneIndex(INDEX_NONE);
			FCompactPoseBoneIndex HipIndex = FCompactPoseBoneIndex(INDEX_NONE);
			float LimbLength = 0.0f;
			float FootLength = 0.0f;
		} Bones;

		FName SpeedCurveName = NAME_None;
		FName DisableLockCurveName = NAME_None;
		FName DisableLegCurveName = NAME_None;

		// Helper struct to store values coming directly, or trivial to calculate from just the input pose.
		struct FInputPoseData
		{
			FTransform FootFKTransformCS = FTransform::Identity;
			FTransform FootTransformCS = FTransform::Identity;
			FTransform BallTransformCS = FTransform::Identity;
			FTransform HipTransformCS = FTransform::Identity;
			FTransform BallToFoot = FTransform::Identity;
			FTransform FootToBall = FTransform::Identity;
#if ENABLE_ANIM_DEBUG
			// These are only used for debug draw at the moment
			// @TODO: Use this info to more precisely figure out foot dimensions
			FTransform FootToGround = FTransform::Identity;
			FTransform BallToGround = FTransform::Identity;
#endif
			float Speed = 0.0f;
			float DisableLeg = 0.0f;
			float LockAlpha = 0.0f;
			float DistanceToPlant = 0.0f;
			// Calculated from a range of toe speeds to define when to blend in/out ground rotational alignment
			// @TODO: When we have prediction/phase info, replac with roll-phase alpha
			float AlignmentAlpha = 0.0f;
		} InputPose;

		/* Ground */

		struct FPlantData
		{
			UE::Anim::FootPlacement::EPlantType PlantType = UE::Anim::FootPlacement::EPlantType::Unplanted;
			UE::Anim::FootPlacement::EPlantType LastPlantType = UE::Anim::FootPlacement::EPlantType::Unplanted;
			FPlane PlantPlaneRS = FPlane(FVector::UpVector, 0.0f);
			FQuat TwistCorrection = FQuat::Identity;
			// @TODO: When we have prediction/phase info, replace use-cases with post-plant roll-phase
			float TimeSinceFullyUnaligned = 0.0f;
			// Whether the planted/locked target has ever been reachable this plant
			bool bCanReachTarget = false;
			// Whether we want to plant, independently from any dynamic pose adjustments we may do
			bool bWantsToPlant = false;

			FPlane GetPlantPlaneCS(const FTransform& RootToComponent) const { return PlantPlaneRS.TransformBy(RootToComponent.ToMatrixWithScale());}
			FPlane GetPlantPlaneWS(const FTransform& RootToComponent, const FTransform& ComponentToWorld) const 
			{ 
				return GetPlantPlaneCS(RootToComponent).TransformBy(ComponentToWorld.ToMatrixWithScale());
			}

		} Plant;
		
		// Ground-aligned, locked/unlocked bone transform pre-extension adjustments
		FTransform AlignedFootTransformWS = FTransform::Identity;
		FTransform AlignedFootTransformRS = FTransform::Identity;
		// Foot locked/unlocked bone transform, before ground alignment
		FTransform UnalignedFootTransformWS = FTransform::Identity;
		FTransform UnalignedFootTransformRS = FTransform::Identity;
		
		/* Interpolation */
		struct FInterpolationData
		{
			// Interpolated foot lock offset
			FTransform UnalignedFootOffset = FTransform::Identity;
			// Separating plane spring states
			FVectorSpringState SeparatingPlaneOffsetSpringState;
			FVector SeparatingPlaneOffset = FVector::ZeroVector;
			// Foot lock spring states
			FVectorSpringState PlantOffsetTranslationSpringState;
			FQuaternionSpringState PlantOffsetRotationSpringState;
			// Ground alignment spring states
			FFloatSpringState GroundHeightSpringState;
			FQuaternionSpringState GroundRotationSpringState;
		} Interpolation;
	};

	struct FPlantRuntimeSettings
	{
		float UnplantRadiusSqrd = 0.0f;
		float ReplantRadiusSqrd = 0.0f;
		float CosHalfUnplantAngle = 0.0f;
		float CosHalfReplantAngle = 0.0f;
	};

	struct FPelvisRuntimeData
	{
		/* Bone IDs */
		struct FBones
		{
			FCompactPoseBoneIndex FkBoneIndex = FCompactPoseBoneIndex(INDEX_NONE);
			FCompactPoseBoneIndex IkBoneIndex = FCompactPoseBoneIndex(INDEX_NONE);
		} Bones;

		/* Settings-based properties */
		float MaxOffsetSqrd = 0.0f;

		/* Input pose properties */
		struct FInputPoseData
		{
			FTransform FKTransformCS = FTransform::Identity;
			FTransform IKRootTransformCS = FTransform::Identity;
			FTransform RootTransformCS = FTransform::Identity;
			FVector FootMidpointCS = FVector::ZeroVector;
		} InputPose;

		/* Interpolation */
		struct FInterpolationData
		{
			// Current pelvis offset and spring states. We use a 3d vector because this interpolates weight rebalancing too.
			FVector PelvisTranslationOffset = FVector::ZeroVector;
			FVectorSpringState PelvisTranslationSpringState;
		} Interpolation;

		
		float DisablePelvis = 0.0f;
	};

	struct FCharacterData
	{
		FTransform ComponentTransformWS = FTransform::Identity;
		FVector ComponentMoveDeltaWS = FVector::ZeroVector;
		FVector CharacterVelocityWS = FVector::ZeroVector;
		FVector SmoothCapsuleGroundNormalWS = FVector::ZeroVector;
		FQuaternionSpringState SmoothCapsuleGroundNormalSpringState;
		bool bIsOnGround = false;
	};

	// Final result after post-adjustments (extension checks, heel lift, etc.)
	struct FPlantResult
	{
	public:
		FBoneTransform FootTranformCS;
		// @TODO: Add procedural toe rolling
		//FBoneTransform BallTransformCS;
		// @TODO: Look into shifting/rotating the hips to prevent over-extension, and give IK an easier time.
		//FBoneTransform HipTransformCS;
	};

#if ENABLE_ANIM_DEBUG
	struct FDebugData
	{
		FVector OutputPelvisLocationWS = FVector::ZeroVector;
		FVector InputPelvisLocationWS = FVector::ZeroVector;

		TArray<FVector> OutputFootLocationsWS;
		TArray<FVector> InputFootLocationsWS;

		struct FLegInfo
		{
			float HyperExtensionAmount;
			float RollAmount;
			float PullAmount;
			float DistanceToSeparatingPlane;
		};
		TArray<FLegInfo> LegsInfo;

		void Init(const int32 InSize)
		{
			OutputFootLocationsWS.SetNumUninitialized(InSize);
			InputFootLocationsWS.SetNumUninitialized(InSize);
			LegsInfo.SetNumUninitialized(InSize);
		}
	};
#endif

	struct FEvaluationContext;
}

UENUM(BlueprintType)
enum class EFootPlacementLockType : uint8
{
	// Foot is unlocked but free to move
	Unlocked,
	// Foot can lock, and will pivot around its ball/toes.
	PivotAroundBall,
	// Foot can lock, and will pivot around the ankle/foot bone.
	PivotAroundAnkle,
	// Foot is fully locked. Useful for bigger/mechanical creatures
	LockRotation
	// @TODO: Detect whether the ball or ankle is planted. Shift the rotation pivot-style depending on this.
};

USTRUCT(BlueprintType)
struct FFootPlacementInterpolationSettings
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Plant Settings")
	float UnplantLinearStiffness = 250.0f;

	UPROPERTY(EditAnywhere, Category = "Plant Settings")
	float UnplantLinearDamping = 1.0f;

	UPROPERTY(EditAnywhere, Category = "Plant Settings")
	float UnplantAngularStiffness = 450.0f;

	UPROPERTY(EditAnywhere, Category = "Plant Settings")
	float UnplantAngularDamping = 1.0f;
	
	UPROPERTY(EditAnywhere, Category = "Plant Settings", meta=(EditCondition="bEnableSeparationInterpolation", DisplayAfter="bEnableSeparationInterpolation"))
	float SeparationStiffness = 1000.0f;

	UPROPERTY(EditAnywhere, Category = "Plant Settings", meta=(EditCondition="bEnableSeparationInterpolation", DisplayAfter="bEnableSeparationInterpolation"))
	float SeparationDamping = 1.0f;

	UPROPERTY(EditAnywhere, Category = "Plant Settings", meta=(EditCondition="bEnableFloorInterpolation", DisplayAfter="bEnableFloorInterpolation"))
	float FloorLinearStiffness = 1000.0f;

	UPROPERTY(EditAnywhere, Category = "Plant Settings", meta=(EditCondition="bEnableFloorInterpolation", DisplayAfter="bEnableFloorInterpolation"))
	float FloorLinearDamping = 1.0f;

	UPROPERTY(EditAnywhere, Category = "Plant Settings", meta=(EditCondition="bEnableFloorInterpolation", DisplayAfter="bEnableFloorInterpolation"))
	float FloorAngularStiffness = 450.0f;

	UPROPERTY(EditAnywhere, Category = "Plant Settings", meta=(EditCondition="bEnableFloorInterpolation", DisplayAfter="bEnableFloorInterpolation"))
	float FloorAngularDamping = 1.0f;

	UPROPERTY(EditAnywhere, Category = "Plant Settings")
	bool bEnableFloorInterpolation = true;
	
	UPROPERTY(EditAnywhere, Category = "Plant Settings")
	bool bSmoothRootBone = false;

	UPROPERTY(EditAnywhere, Category = "Plant Settings")
	bool bEnableSeparationInterpolation = true;
};

USTRUCT(BlueprintType)
struct FFootPlacementTraceSettings
{
	GENERATED_BODY()

public:

	// A negative value extends the trace length above the bone
	UPROPERTY(EditAnywhere, Category = "Trace Settings", meta=(EditCondition="bEnabled", DisplayAfter="bEnabled"))
	float StartOffset = -75.0f;

	// A positive value extends the trace length below the bone
	UPROPERTY(EditAnywhere, Category = "Trace Settings", meta=(EditCondition="bEnabled", DisplayAfter="bEnabled"))
	float EndOffset = 100.0f;

	// The trace is a sphere sweep with this radius. It should be big enough to prevent the trace from going through 
	// small geometry gaps
	UPROPERTY(EditAnywhere, Category = "Trace Settings", meta=(EditCondition="bEnabled", DisplayAfter="bEnabled"))
	float SweepRadius = 5.0f;

	UPROPERTY(EditAnywhere, Category = "Trace Settings", meta=(EditCondition="bEnabled", DisplayAfter="bEnabled"))
	bool bDisableComplexTrace = false;

	// The channel to use for our complex trace
	UPROPERTY(EditAnywhere, Category = "Trace Settings", meta=(EditCondition="!bDisableComplexTrace", DisplayAfter="bDisableComplexTrace"))
	TEnumAsByte<ETraceTypeQuery> ComplexTraceChannel = TraceTypeQuery1;

	// How much the feet can penetrate the ground geometry. It's recommended to allow some to account for interpolation
	// Negative values disable this effect
	UPROPERTY(EditAnywhere, Category = "Trace Settings")
	float MaxGroundPenetration = 10.0f;

	// The channel to use for our simple trace
	UPROPERTY(EditAnywhere, Category = "Trace Settings", meta = (EditCondition = "bEnabled", DisplayAfter = "bEnabled"))
	TEnumAsByte<ETraceTypeQuery> SimpleTraceChannel = TraceTypeQuery1;

	// Enabling tracing for ground alignment
	// @TODO: Use ground normal when not tracing
	UPROPERTY(EditAnywhere, Category = "Trace Settings")
	bool bEnabled = true;
};

USTRUCT()
struct FFootPlacementRootDefinition
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Pelvis Settings")
	FBoneReference PelvisBone;

	UPROPERTY(EditAnywhere, Category = "Pelvis Settings")
	FBoneReference IKRootBone;

public:
	void Initialize(const FAnimationInitializeContext& Context);
};

UENUM(BlueprintType)
enum class EPelvisHeightMode : uint8
{
	// Consider all legs for pelvis height, whether they're planted or not
	// Generally good for flat/not too steep ground
	AllLegs,
	// Consider only the planted feet when calculating pelvis height
	// Generally good we pelvis height to be defined by the weight supporting leg
	AllPlantedFeet,
	// When moving uphill, use the front foot, as long as it's planted.
	// It's recommended to also enable pelvis interpolation to smooth out the swap between what's considered the "planted" leg
	// When moving downhill, both feet will be considered relevant.
	// The algorithm tends to prefer the lower foot, except when the higher foot would become over-compresseed.
	FrontPlantedFeetUphill_FrontFeetDownhill,
};

UENUM(BlueprintType)
enum class EActorMovementCompensationMode : uint8
{
	// Keep pelvis component-space and follow along all of the actor's vertical ground movement. Use when you have really smooth ground geometry, or if you have moving platforms.
	ComponentSpace,
	// Hold pelvis world-space and ignore the actor's vertical ground movement. Let springs interpolate the difference
	WorldSpace,
	// Keep pelvis component-space, but hold world-space transform when the actor does sudden changes (i.e. a big step), and let springs interpolate the difference. Does not work on moving platforms.
	SuddenMotionOnly
};

USTRUCT(BlueprintType)
struct FFootPlacementPelvisSettings
{
	GENERATED_BODY()

public:
	// Max vertical offset from the input pose for the Pelvis.
	// Reaching this limit means the feet may not reach their plant plane
	UPROPERTY(EditAnywhere, Category = "Pelvis Settings")
	float MaxOffset = 50.0f;

	UPROPERTY(EditAnywhere, Category = "Pelvis Settings", meta=(EditCondition="bEnableInterpolation", DisplayAfter="bEnableInterpolation"))
	float LinearStiffness = 350.0f;

	UPROPERTY(EditAnywhere, Category = "Pelvis Settings", meta=(EditCondition="bEnableInterpolation", DisplayAfter="bEnableInterpolation"))
	float LinearDamping = 1.0f;

	// How much we move the pelvis horizontally to re-balance the characters weight due to foot offsets.
	// A value of 0 will disable this effect.
	// Higher values may move the mesh outside of the character's capsule
	UPROPERTY(EditAnywhere, Category = "Pelvis Settings", meta = (ClampMin = "0.0", UIMin = "0.0", ClampMax = "1.0", UIMax = "1.0", DisplayAfter="bEnableInterpolation"))
	float HorizontalRebalancingWeight = 0.3f;

	// Max horizontal foot adjustment we consider to lower the hips
	// This can be used to prevent the hips from dropping too low when the feet are locked
	// Exceeding this value will first attempt to roll the planted feet, and then slide
	UPROPERTY(EditAnywhere, Category = "Pelvis Settings", meta=(DisplayAfter="bEnableInterpolation"))
	float MaxOffsetHorizontal = 10.0f;

	// How much we prefer lifting the heel before dropping the hips to achieve the desired pose.
	// 1 fully favors heel lift first.
	// 0 fully favors pelvis drop first.
	UPROPERTY(EditAnywhere, Category = "Pelvis Settings", meta = (ClampMin = "0.0", UIMin = "0.0", ClampMax = "1.0", UIMax = "1.0", DisplayAfter="bEnableInterpolation"))
	float HeelLiftRatio = 0.5f;

	UPROPERTY(EditAnywhere, Category = "Pelvis Settings", meta=(DisplayAfter="bEnableInterpolation"))
	EPelvisHeightMode PelvisHeightMode = EPelvisHeightMode::AllLegs;

	// This is used to hold the Pelvis's interpolator in a fixed spot when the capsule moves up/down
	// If your camera is directly attached to the character with little to no smoothing, you may want this on ComponentSpace
	UPROPERTY(EditAnywhere, Category = "Pelvis Settings", meta=(DisplayAfter="bEnableInterpolation"))
	EActorMovementCompensationMode ActorMovementCompensationMode = EActorMovementCompensationMode::SuddenMotionOnly;

	UPROPERTY(EditAnywhere, Category = "Pelvis Settings")
	bool bEnableInterpolation = true;

	UPROPERTY(EditAnywhere, Category = "Pelvis Settings")
	bool bDisablePelvisOffsetInAir = true;
	
	UPROPERTY(EditAnywhere, Category = "Pelvis Settings")
	FName DisablePelvisCurveName = NAME_None;
};

USTRUCT()
struct FFootPlacemenLegDefinition
{
	GENERATED_BODY()

public:
	// Bone to be planted. For feet, use the heel/ankle joint.
	UPROPERTY(EditAnywhere, Category = "Settings")
	FBoneReference FKFootBone;

	// TODO: can we optionally output as an attribute?
	UPROPERTY(EditAnywhere, Category = "Settings")
	FBoneReference IKFootBone;

	// Secondary plant bone. For feet, use the ball joint.
	UPROPERTY(EditAnywhere, Category = "Settings")
	FBoneReference BallBone;

	UPROPERTY(EditAnywhere, Category = "Settings")
	int32 NumBonesInLimb = 2;

	// Name of the curve representing the foot/ball speed. Not required in Graph speed mode
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName SpeedCurveName = NAME_None;

	// Name of the curve representing the alpha of the locking alpha.
	// This allows you to disable locking precisely, instead of relying on the procedural mechanism based on springs and foot analysis
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName DisableLockCurveName = NAME_None;
	
	// Curve to disable the effect of footplacement on this leg
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName DisableLegCurveName = NAME_None;

public:

	void InitializeBoneReferences(const FBoneContainer& RequiredBones);
};

USTRUCT(BlueprintType)
struct FFootPlacementPlantSettings
{
	GENERATED_BODY()

public:
	
	// Bone is considered planted below this speed. Value is obtained from FKSpeedCurveName
	UPROPERTY(EditAnywhere, Category = "Plant Settings")
	float SpeedThreshold = 60.0f;

	// At this distance from the planting plane the bone is considered planted and will be fully aligned.
	UPROPERTY(EditAnywhere, Category = "Plant Settings")
	float DistanceToGround = 10.0f;

	UPROPERTY(EditAnywhere, Category = "Plant Settings")
	EFootPlacementLockType LockType = EFootPlacementLockType::PivotAroundBall;

	// How much linear deviation causes the constraint to be released
	UPROPERTY(EditAnywhere, Category = "Plant Settings")
	float UnplantRadius = 35.0f;

	// Below this value, proportional to UnplantRadius, the bone will replant
	UPROPERTY(EditAnywhere, Category = "Plant Settings", meta = (ClampMin = "0.0", UIMin = "0.0", ClampMax = "1.0", UIMax = "1.0"))
	float ReplantRadiusRatio = 0.35f;

	// How much angular deviation (in degrees) causes the constraint to be released for replant
	UPROPERTY(EditAnywhere, Category = "Plant Settings")
	float UnplantAngle = 45.0;

	// Below this value, proportional to UnplantAngle, the bone will replant
	UPROPERTY(EditAnywhere, Category = "Plant Settings", meta = (ClampMin = "0.0", UIMin = "0.0", ClampMax = "1.0", UIMax = "1.0"))
	float ReplantAngleRatio = 0.5f;

	// Max extension ratio of the chain, calculated from the remaining length between current pose and full limb extension
	UPROPERTY(EditAnywhere, Category = "Plant Settings", meta = (ClampMin = "0.0", UIMin = "0.0", ClampMax = "1.0", UIMax = "1.0"))
	float MaxExtensionRatio = 0.5f;

	// Min extension ratio of the chain, calculated from the total limb length, and adjusted along the approach direction
	UPROPERTY(EditAnywhere, Category = "Plant Settings", meta = (ClampMin = "0.0", UIMin = "0.0", ClampMax = "1.0", UIMax = "1.0"))
	float MinExtensionRatio = 0.2f;

	// The minimum distance the feet can be from the plane that separates the feet. 
	// Value of 0 disables this
	UPROPERTY(EditAnywhere, Category = "Plant Settings", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float SeparatingDistance = 0.0f;
	
	// Speed at which we transition to fully unplanted.
	// The range between SpeedThreshold and UnalignmentSpeedThreshold should roughly represent the roll-phase of the foot
	// TODO: This feels innaccurate most of the time, and varies depending on anim speed. Improve this
	UPROPERTY(EditAnywhere, Category = "Plant Settings")
	float UnalignmentSpeedThreshold = 200.0f;

	// How much we reduce the procedural ankle twist adjustment used to align the foot to the ground slope.
	UPROPERTY(EditAnywhere, Category = "Plant Settings", meta = (ClampMin = "0.0", UIMin = "0.0", ClampMax = "1.0", UIMax = "1.0"))
	float AnkleTwistReduction = 0.75f;

	// Only consider world deltas caused by played velocity. This is useful when on a moving platform i.e. but won't consider slopes.
	UPROPERTY(EditAnywhere, Category = "Plant Settings")
	bool bReconstructWorldPlantFromVelocity = false;

	// Whether to allow adjusting the heel lift before we plant
	UPROPERTY(EditAnywhere, Category = "Plant Settings")
	bool bAdjustHeelBeforePlanting = false;

	void Initialize(const FAnimationInitializeContext& Context);
};


USTRUCT(BlueprintInternalUseOnly, Experimental)
struct FAnimNode_FootPlacement : public FAnimNode_SkeletalControlBase
{
	GENERATED_BODY()

public:

	// Foot/Ball speed evaluation mode (Graph or Manual) used to decide when the feet are locked
	// Graph mode uses the root motion attribute from the animations to calculate the joint's speed
	// Manual mode uses a per-foot curve name representing the joint's speed
	UPROPERTY(EditAnywhere, Category = "Settings")
	EWarpingEvaluationMode PlantSpeedMode = EWarpingEvaluationMode::Manual;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FBoneReference IKFootRootBone;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FBoneReference PelvisBone;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (PinHiddenByDefault))
	FFootPlacementPelvisSettings PelvisSettings;

	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<FFootPlacemenLegDefinition> LegDefinitions;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (PinHiddenByDefault))
	FFootPlacementPlantSettings PlantSettings;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (PinHiddenByDefault))
	FFootPlacementInterpolationSettings InterpolationSettings;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (PinHiddenByDefault))
	FFootPlacementTraceSettings TraceSettings;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	FVector BaseTranslationDelta = FVector::ZeroVector;

public:
	UE_API FAnimNode_FootPlacement();

	// FAnimNode_Base interface
	UE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	UE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	UE_API virtual void UpdateInternal(const FAnimationUpdateContext& Context) override;
	UE_API virtual void EvaluateSkeletalControl_AnyThread(
		FComponentSpacePoseContext& Output, 
		TArray<FBoneTransform>& OutBoneTransforms) override;
	UE_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase

private:
	// FAnimNode_SkeletalControlBase interface
	UE_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

	// Gather raw or trivially calculated values from input pose
	void GatherPelvisDataFromInputs(const UE::Anim::FootPlacement::FEvaluationContext& Context);
	void GatherLegDataFromInputs(
		const UE::Anim::FootPlacement::FEvaluationContext& Context,
		UE::Anim::FootPlacement::FLegRuntimeData& LegData,
		const FFootPlacemenLegDefinition& LegDef);

	void CalculateFootMidpoint(
		const UE::Anim::FootPlacement::FEvaluationContext& Context,
		TConstArrayView<UE::Anim::FootPlacement::FLegRuntimeData> LegData,
		FVector& OutMidpoint) const;

	// Calculate procedural adjustments before solving the desired pelvis position
	void ProcessCharacterState(const UE::Anim::FootPlacement::FEvaluationContext& Context);
	void ProcessFootAlignment(
		const UE::Anim::FootPlacement::FEvaluationContext& Context,
		UE::Anim::FootPlacement::FLegRuntimeData& LegData);

	// Calculate the desired pelvis offset, based on procedural character/foot adjustments
	FTransform SolvePelvis(const UE::Anim::FootPlacement::FEvaluationContext& Context);

	FTransform UpdatePelvisInterpolationRootSpace(
		const UE::Anim::FootPlacement::FEvaluationContext& Context,
		const FTransform& TargetPelvisTransform);

	// Post-processing adjustments + fix hyper-extension/compression
	UE::Anim::FootPlacement::FPlantResult FinalizeFootAlignment(
		const UE::Anim::FootPlacement::FEvaluationContext& Context,
		UE::Anim::FootPlacement::FLegRuntimeData& LegData,
		const FFootPlacemenLegDefinition& LegDef,
		const FTransform& PelvisTransformCS);

	FVector GetApproachDirWS(const FAnimationBaseContext& Context) const;

	const FTransform& GetRootToComponent() const;

private:
	float CachedDeltaTime = 0.0f;
	FVector LastComponentLocation = FVector::ZeroVector;

	TArray<UE::Anim::FootPlacement::FLegRuntimeData> LegsData;
	UE::Anim::FootPlacement::FPlantRuntimeSettings PlantRuntimeSettings;
	UE::Anim::FootPlacement::FPelvisRuntimeData PelvisData;
	UE::Anim::FootPlacement::FCharacterData CharacterData;

	// Whether we want to plant, independently from any dynamic pose adjustments we may do
	bool WantsToPlant(
		const UE::Anim::FootPlacement::FEvaluationContext& Context,
		const UE::Anim::FootPlacement::FLegRuntimeData::FInputPoseData& LegInputPose) const;

	// Get Alignment Alpha based on current foot speed
	// 0.0 is fully unaligned and the foot is in flight.
	// 1.0 is fully aligned and the foot is planted.
	float GetAlignmentAlpha(
		const UE::Anim::FootPlacement::FEvaluationContext& Context,
		const UE::Anim::FootPlacement::FLegRuntimeData::FInputPoseData& LegInputPose) const;

	// This function looks at both the foot bone and the ball bone, returning the smallest distance to the
	// planting plane. Note this distance can be negative, meaning it's penetrating.
	float CalcTargetPlantPlaneDistance(
		const UE::Anim::FootPlacement::FEvaluationContext& Context,
		const UE::Anim::FootPlacement::FLegRuntimeData::FInputPoseData& LegInputPose) const;

	struct FPelvisOffsetRangeForLimb
	{
		float MaxExtension;
		float MinExtension;
		float DesiredExtension;
	};

	// Find the horizontal pelvis offset range for the foot to reach:
	void FindPelvisOffsetRangeForLimb(
		const UE::Anim::FootPlacement::FEvaluationContext& Context,
		const UE::Anim::FootPlacement::FLegRuntimeData& LegData,
		const FVector& PlantTargetLocationCS,
		const FTransform& PelvisTransformCS,
		FPelvisOffsetRangeForLimb& OutPelvisOffsetRangeCS) const;

	// Adjust LastPlantTransformWS to current, to have the foot pivot around the ball instead of the ankle
	FTransform GetFootPivotAroundBallWS(
		const UE::Anim::FootPlacement::FEvaluationContext& Context,
		const UE::Anim::FootPlacement::FLegRuntimeData::FInputPoseData& LegInputPose,
		const FTransform& LastPlantTransformWS) const;

	// Align the transform the provided world space ground plant plane.
	// Also outputs the twist along the ground plane needed to get there
	void AlignPlantToGround(
		const UE::Anim::FootPlacement::FEvaluationContext& Context,
		const FPlane& PlantPlaneWS,
		const UE::Anim::FootPlacement::FLegRuntimeData::FInputPoseData& LegInputPose,
		FTransform& InOutFootTransformWS,
		FQuat& OutTwistCorrection) const;

	// Handles horizontal interpolation when unlocking the plant
	FTransform UpdatePlantOffsetInterpolation(
		const UE::Anim::FootPlacement::FEvaluationContext& Context,
		UE::Anim::FootPlacement::FLegRuntimeData::FInterpolationData& InOutInterpData) const;

	// Handles the interpolation of the planting plane. Because the plant transform is specified with respect to the 
	// planting plane, it cannot change abruptly without causing an animation pop. It must be interpolated instead.
	void UpdatePlantingPlaneInterpolation(
		const UE::Anim::FootPlacement::FEvaluationContext& Context,
		const FTransform& FootTransformWS,
		const FTransform& LastAlignedFootTransform,
		const float AlignmentAlpha,
		FPlane& InOutPlantPlane,
		const UE::Anim::FootPlacement::FLegRuntimeData::FInputPoseData& LegInputPose,
		UE::Anim::FootPlacement::FLegRuntimeData::FInterpolationData& InOutInterpData) const;

	// Checks unplanting and replanting conditions to determine if the foot is planted
	void DeterminePlantType(
		const UE::Anim::FootPlacement::FEvaluationContext& Context,
		const FTransform& FKTransformWS,
		const FTransform& CurrentBoneTransformWS,
		UE::Anim::FootPlacement::FLegRuntimeData::FPlantData& InOutPlantData,
		const UE::Anim::FootPlacement::FLegRuntimeData::FInputPoseData& LegInputPose) const;
	
	float GetMaxLimbExtension(const float DesiredExtension, const float LimbLength) const;
	float GetMinLimbExtension(const float DesiredExtension, const float LimbLength) const;

	void ResetRuntimeData();

	
#if ENABLE_FOOTPLACEMENT_DEBUG
	UE::Anim::FootPlacement::FDebugData DebugData;

	void DrawDebug(
		const UE::Anim::FootPlacement::FEvaluationContext& Context,
		const UE::Anim::FootPlacement::FLegRuntimeData& LegData,
		const UE::Anim::FootPlacement::FPlantResult& PlantResult) const;

	void DrawVLog(
		const UE::Anim::FootPlacement::FEvaluationContext& Context,
		const UE::Anim::FootPlacement::FLegRuntimeData& LegData,
    	const UE::Anim::FootPlacement::FPlantResult& PlantResult) const;
#endif

	bool bIsFirstUpdate = false;
	FGraphTraversalCounter UpdateCounter;
};

#undef UE_API
