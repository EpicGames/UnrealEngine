// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoneControllers/BoneControllerTypes.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "AnimationWarpingTypes.h"
#include "AnimNode_OffsetRootBone.generated.h"

#define UE_API ANIMATIONWARPINGRUNTIME_API

struct FAnimationInitializeContext;
struct FComponentSpacePoseContext;
struct FNodeDebugData;

namespace UE::AnimationWarping
{
class FRootOffsetProvider : public UE::Anim::IGraphMessage
{
	DECLARE_ANIMGRAPH_MESSAGE_API(FRootOffsetProvider, UE_API);

public:

	FRootOffsetProvider(const FTransform& InRootTransform)
		: RootTransform(InRootTransform)
	{
	}
	
	const FTransform& GetRootTransform() const { return RootTransform; } 

private:
	FTransform RootTransform;
};
}

UENUM(BlueprintType)
enum class EOffsetRootBone_CollisionTestingMode : uint8
{
	// No Collision testing
	Disabled,
	// Reduce effective Max Translation offset to prevent penetration with nearby obstacles
	ShrinkMaxTranslation,
	// Slide along a plane based on shape cast contact point
	PlanarCollision,
};

USTRUCT(BlueprintInternalUseOnly, Experimental)
struct FAnimNode_OffsetRootBone : public FAnimNode_Base
{
	GENERATED_BODY();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Links, meta = (DisplayPriority = 0))
	FPoseLink Source;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = Evaluation, meta=(FoldProperty))
	EWarpingEvaluationMode EvaluationMode = EWarpingEvaluationMode::Graph;
	
	UPROPERTY(EditAnywhere, Category = Settings, meta = (FoldProperty, PinHiddenByDefault))
	bool bResetEveryFrame = false;

	// The translation offset behavior mode
	UPROPERTY(EditAnywhere, Category = Settings, meta = (FoldProperty, PinHiddenByDefault))
	EOffsetRootBoneMode TranslationMode = EOffsetRootBoneMode::Interpolate;

	// The rotation offset behavior mode
	UPROPERTY(EditAnywhere, Category = Settings, meta = (FoldProperty, PinHiddenByDefault))
	EOffsetRootBoneMode RotationMode = EOffsetRootBoneMode::Interpolate;

	// Controls how fast the translation offset is blended out
	// Values closer to 0 make it faster
	UPROPERTY(EditAnywhere, Category = Settings, meta = (FoldProperty, PinHiddenByDefault))
	float TranslationHalflife = 0.1f;

	// Controls how fast the rotation offset is blended out
	// Values closer to 0 make it faster
	UPROPERTY(EditAnywhere, Category = Settings, meta = (FoldProperty, PinHiddenByDefault))
	float RotationHalfLife = 0.2f;

	// How much the offset can deviate from the mesh component's translation in units
	// Values lower than 0 disable this limit
	UPROPERTY(EditAnywhere, Category = Settings, meta = (FoldProperty, PinHiddenByDefault))
	float MaxTranslationError = -1.0f;

	// How much the offset can deviate from the mesh component's rotation in degrees
	// Values lower than 0 disable this limit
	UPROPERTY(EditAnywhere, Category = Settings, meta = (FoldProperty, PinHiddenByDefault))
	float MaxRotationError = -1.0f;

	// Whether to limit the offset's translation interpolation speed to the velocity on the incoming motion
	// Enabling this prevents the offset sliding when there's little to no translation speed
	UPROPERTY(EditAnywhere, Category = Settings, meta = (FoldProperty, PinHiddenByDefault))
	bool bClampToTranslationVelocity = false;

	// Whether to limit the offset's rotation interpolation speed to the velocity on the incoming motion
	// Enabling this prevents the offset sliding when there's little to no rotation speed
	UPROPERTY(EditAnywhere, Category = Settings, meta = (FoldProperty, PinHiddenByDefault))
	bool bClampToRotationVelocity = false;

	// How much the offset can blend out, relative to the incoming translation speed
	// i.e. If root motion is moving at 400cm/s, at 0.5, the offset can blend out at 200cm/s
	UPROPERTY(EditAnywhere, Category = Settings, meta = (EditCondition = bClampToTranslationVelocity, FoldProperty, PinHiddenByDefault))
	float TranslationSpeedRatio = 0.5f;

	// How much the offset can blend out, relative to the incoming rotation speed
	// i.e. If root motion is rotating at 90 degrees/s, at 0.5, the offset can blend out at 45 degree/s
	UPROPERTY(EditAnywhere, Category = Settings, meta = (EditCondition = bClampToRotationVelocity, FoldProperty, PinHiddenByDefault))
	float RotationSpeedRatio = 0.5f;

	// When OnGround is true, root motion velocities will be projected onto the ground surface
	UPROPERTY(EditAnywhere, Category = Settings, meta = (FoldProperty, PinHiddenByDefault))
	bool bOnGround = true;
	
	// Surface normal of the ground
	UPROPERTY(EditAnywhere, Category = Settings, meta = (FoldProperty, PinHiddenByDefault))
	FVector GroundNormal = { 0, 0, 1 };

	// Delta applied to the translation offset this frame. 
	// For procedural values, consider adjusting the input by delta time.
	UPROPERTY(EditAnywhere, Category = Evaluation, meta = (FoldProperty, PinHiddenByDefault))
	FVector TranslationDelta = FVector::ZeroVector;

	// Delta applied to the rotation offset this frame. 
	// For procedural values, consider adjusting the input by delta time.
	UPROPERTY(EditAnywhere, Category = Evaluation, meta = (FoldProperty, PinHiddenByDefault))
	FRotator RotationDelta = FRotator::ZeroRotator;

	// If to reset the root bone offset on teleport
	UPROPERTY(EditAnywhere, Category = Settings, meta = (FoldProperty, PinHiddenByDefault))
	bool bResetOnTeleport = false;

	UPROPERTY(EditAnywhere, Category = CollisionTesting, meta = (FoldProperty, PinHiddenByDefault))
	EOffsetRootBone_CollisionTestingMode CollisionTestingMode = EOffsetRootBone_CollisionTestingMode::Disabled;
	UPROPERTY(EditAnywhere, Category = CollisionTesting, meta = (EditCondition = "CollisionTestingMode != ECollisionTestingMode::Disabled", DisplayAfter="CollisionTestingMode", FoldProperty, PinHiddenByDefault))
	float CollisionTestShapeRadius = 30;
	UPROPERTY(EditAnywhere, Category = CollisionTesting, meta = (EditCondition = "CollisionTestingMode != ECollisionTestingMode::Disabled", DisplayAfter="CollisionTestingMode", FoldProperty, PinHiddenByDefault))
	FVector CollisionTestShapeOffset = {0,0,60};
			
#endif

public:
	// FAnimNode_Base interface
	UE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	UE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	UE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	UE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	UE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	// End of FAnimNode_Base interface

	// Folded property accesors
	UE_API EWarpingEvaluationMode GetEvaluationMode() const;
	UE_API bool GetResetEveryFrame() const;
	UE_API bool GetOnGround() const;
	UE_API const FVector& GetGroundNormal() const;
	UE_API const FVector& GetTranslationDelta() const;
	UE_API const FRotator& GetRotationDelta() const;
	UE_API EOffsetRootBoneMode GetTranslationMode() const;
	UE_API EOffsetRootBoneMode GetRotationMode() const;
	UE_API float GetTranslationHalflife() const;
	UE_API float GetRotationHalfLife() const;
	UE_API float GetMaxTranslationError() const;
	UE_API float GetMaxRotationError() const;
	UE_API bool GetClampToTranslationVelocity() const;
	UE_API bool GetClampToRotationVelocity() const;
	UE_API float GetTranslationSpeedRatio() const;
	UE_API float GetRotationSpeedRatio() const;
	UE_API EOffsetRootBone_CollisionTestingMode GetCollisionTestingMode() const;
	UE_API float GetCollisionTestShapeRadius() const;
	UE_API const FVector& GetCollisionTestShapeOffset() const;
	UE_API bool GetResetOnTeleport() const;

	// get the current simulated root transform
	void GetOffsetRootTransform(FTransform& OutTransform)
	{
		OutTransform.SetRotation(SimulatedRotation);
		OutTransform.SetTranslation(SimulatedTranslation);
	}

	const FVector& GetOffsetRootTranslation() const { return SimulatedTranslation; }
	void SetOffsetRootTranslation(const FVector& InTranslation) { SimulatedTranslation = InTranslation; }
	
	const FQuat& GetOffsetRootRotation() const { return SimulatedRotation; };
	void SetOffsetRootRotation(const FQuat& InRotation) { SimulatedRotation = InRotation; }

private:

	UE_API void Reset(const FAnimationBaseContext& Context);

	// Internal cached anim instance proxy
	FAnimInstanceProxy* AnimInstanceProxy = nullptr;

	// Internal cached delta time used for interpolators
	float CachedDeltaTime = 0.f;

	bool bIsFirstUpdate = true;

	FTransform ComponentTransform = FTransform::Identity;

	// The simulated world-space transforms for the root bone with offset
	// Offset = ComponentTransform - SimulatedTransform
	FVector SimulatedTranslation = FVector::ZeroVector;
	FQuat SimulatedRotation = FQuat::Identity;

	FVector LastNonZeroRootMotionDirection = FVector::ZeroVector;

	FGraphTraversalCounter UpdateCounter;
};

#undef UE_API
