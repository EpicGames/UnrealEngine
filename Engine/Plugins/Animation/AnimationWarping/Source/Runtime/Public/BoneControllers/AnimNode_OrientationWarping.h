// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoneControllers/BoneControllerTypes.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "EngineDefines.h"
#include "AnimNode_OrientationWarping.generated.h"

#define UE_API ANIMATIONWARPINGRUNTIME_API

struct FAnimationInitializeContext;
struct FComponentSpacePoseContext;
struct FNodeDebugData;
class UAnimationAsset;


UENUM(BlueprintType)
enum class EOrientationWarpingSpace : uint8
{
	// apply warping relative to current component transform
	ComponentTransform,
	// Apply warping relative to previous frame's root bone transform. Use this mode when using an OffsetRootBone node which allows the root bone and component transforms to differ. 
	RootBoneTransform,
	// Provide a custom transform pin
	CustomTransform
};

/**
 * Maintains a look at direction for the upper body (orientation), while rotating the lower body to match capsule velocity direction
 * Does nothing if the root motion velocity direction matches the desired / current capsule velocity direction
 */
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_OrientationWarping : public FAnimNode_SkeletalControlBase
{
	GENERATED_BODY()

	// Orientation warping evaluation mode (Graph or Manual)
	UPROPERTY(EditAnywhere, Category=Evaluation)
	EWarpingEvaluationMode Mode = EWarpingEvaluationMode::Manual;

	// Experimental. Orientation warping should do nothing if root motion velocity directions match capsule,
	// however root motion can have multiple velocity directions. So we also check root motion
	// direction 'TargetTime' in the future for matching direction to avoid temp orientation warps.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Experimental, meta = (PinShownByDefault))
	float TargetTime = 0.8f;

	// The desired orientation angle (in degrees) to warp by relative to the specified RotationAxis
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Evaluation, meta=(PinShownByDefault))
	float OrientationAngle = 0.f;

	// The character locomotion angle (in degrees) relative to the specified RotationAxis
	// This will be used in the following equation for computing the orientation angle: [Orientation = RotationBetween(RootMotionDirection, LocomotionDirection)]
	// In most cases, this is the difference between the Velocity of the Movement Component and the actor rotation (obtained via CalculateDirection)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Evaluation, meta=(PinShownByDefault))
	float LocomotionAngle = 0.f;

	// The character movement direction vector in world space
	// When set, this vector is used to compute LocomotionAngle automatically. When not set, the LocomotionAngle input should be used instead.
	// In most cases, this vector is the same as the Velocity vector of the Movement Component
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Evaluation, meta=(PinShownByDefault))
	FVector LocomotionDirection = { 0.f, 0.f, 0.f };
	
	// Minimum root motion speed required to apply orientation warping
	// This is useful to prevent unnatural re-orientation when the animation has a portion with no root motion (i.e starts/stops/idles)
	// When this value is greater than 0, it's recommended to enable interpolation with RotationInterpSpeed > 0
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Evaluation, meta = (ClampMin = "0.0", PinHiddenByDefault))
	float MinRootMotionSpeedThreshold = 10.0f;

	// Specifies an angle threshold to prevent erroneous over-rotation of the character, disabled with a value of 0
	//
	// When the effective orientation warping angle is detected to be greater than this value (default: 90 degrees) the locomotion direction will be inverted prior to warping
	// This will be used in the following equation: [Orientation = RotationBetween(RootMotionDirection, -LocomotionDirection)]
	//
	// Example: Playing a forward running animation while the motion is going backward 
	// Rather than orientation warping by 180 degrees, the system will warp by 0 degrees 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Evaluation, meta=(PinHiddenByDefault), meta=(ClampMin="0.0", ClampMax="180.0"))
	float LocomotionAngleDeltaThreshold = 90.f;

	// Spine bone definitions
	// Used to counter rotate the body in order to keep the character facing forward
	// The amount of counter rotation applied is driven by DistributedBoneOrientationAlpha
	UPROPERTY(EditAnywhere, Category=Settings)
	TArray<FBoneReference> SpineBones;

	// IK Foot Root Bone definition
	UPROPERTY(EditAnywhere, Category=Settings, meta=(DisplayName="IK Foot Root Bone"))
	FBoneReference IKFootRootBone;

	// IK Foot definitions
	UPROPERTY(EditAnywhere, Category=Settings, meta=(DisplayName="IK Foot Bones"))
	TArray<FBoneReference> IKFootBones;

	// Experimental. Animation Asset for incorporating root motion data. If 'TargetTime' is set, and the animation has root motion rotation within the TargetTime, 
	// then those rotations will be scaled to reach the TargetOrientation
	UPROPERTY(EditAnywhere, Transient, BlueprintReadWrite, Category = Experimental, meta = (PinShownByDefault))
	TObjectPtr<UAnimationAsset> CurrentAnimAsset;

	// Experimental. Current playback time in seconds of the CurrentAnimAsset
	UPROPERTY(EditAnywhere, Transient, BlueprintReadWrite, Category = Experimental, meta = (PinShownByDefault))
	float CurrentAnimAssetTime = 0.f;

	// Rotation axis used when rotating the character body
	UPROPERTY(EditAnywhere, Category=Settings)
	TEnumAsByte<EAxis::Type> RotationAxis = EAxis::Z;

	// Specifies how much rotation is applied to the character body versus IK feet
	UPROPERTY(EditAnywhere, Category=Settings, meta=(ClampMin="0.0", ClampMax="1.0", PinHiddenByDefault))
	float DistributedBoneOrientationAlpha = 0.5f;

	// Specifies the interpolation speed (in Alpha per second) towards reaching the final warped rotation angle
	// A value of 0 will cause instantaneous rotation, while a greater value will introduce smoothing
	UPROPERTY(EditAnywhere, Category=Settings, meta=(ClampMin="0.0"))
	float RotationInterpSpeed = 10.f;

	// Same as RotationInterpSpeed, but for CounterCompensate smoothing. A value of 0 sample raw root motion.
	// Used to avoid stuttering from resampling root deltas. Root motion is already smooth, so a large value is our default (~75% of 60 fps).
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0"))
	float CounterCompensateInterpSpeed = 45.f;

	// Max correction we're allowed to do per-second when using interpolation.
	// This minimizes pops when we have a large difference between current and target orientation.
	UPROPERTY(EditAnywhere, Category=Settings, meta=(ClampMin="0.0", EditCondition="RotationInterpSpeed > 0.0f"))
	float MaxCorrectionDegrees = 180.f;

	// Don't compensate our interpolator when the instantaneous root motion delta is higher than this. This is likely a pivot.
	UPROPERTY(EditAnywhere, Category=Settings, meta=(ClampMin="0.0", EditCondition="RotationInterpSpeed > 0.0f"))
	float MaxRootMotionDeltaToCompensateDegrees = 45.f;

	// Whether to counter compensate interpolation by the animated root motion angle change over time.
	// This helps to conserve the motion from our animation.
	// Disable this if your root motion is expected to be jittery, and you want orientation warping to smooth it out.
	UPROPERTY(EditAnywhere, Category=Settings, meta=(EditCondition="RotationInterpSpeed > 0.0f"))
	bool bCounterCompenstateInterpolationByRootMotion = true;

	UPROPERTY(EditAnywhere, Category=Experimental, meta=(PinHiddenByDefault))
	bool bScaleByGlobalBlendWeight = false;

	UPROPERTY(EditAnywhere, Category=Experimental, meta=(PinHiddenByDefault))
	bool bUseManualRootMotionVelocity = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Experimental, meta=(PinHiddenByDefault))
	FVector ManualRootMotionVelocity = FVector::ZeroVector;

	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Evaluation, meta=(PinHiddenByDefault))
	EOrientationWarpingSpace WarpingSpace = EOrientationWarpingSpace::ComponentTransform;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Evaluation, meta=(PinHiddenByDefault))
	FTransform WarpingSpaceTransform;

#if WITH_EDITORONLY_DATA
	// Scale all debug drawing visualization by a factor
	UPROPERTY(EditAnywhere, Category=Debug, meta=(ClampMin="0.0"))
	float DebugDrawScale = 1.f;

	// Enable/Disable orientation warping debug drawing
	UPROPERTY(EditAnywhere, Category=Debug)
	bool bEnableDebugDraw = false;
#endif

public:
	// FAnimNode_Base interface
	UE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	UE_API virtual void UpdateInternal(const FAnimationUpdateContext& Context) override;
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	UE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	UE_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	UE_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

private:
	// FAnimNode_SkeletalControlBase interface
	UE_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

	struct FOrientationWarpingSpineBoneData
	{
		FCompactPoseBoneIndex BoneIndex;
		float Weight;

		FOrientationWarpingSpineBoneData()
			: BoneIndex(INDEX_NONE)
			, Weight(0.f)
		{
		}

		FOrientationWarpingSpineBoneData(FCompactPoseBoneIndex InBoneIndex)
			: BoneIndex(InBoneIndex)
			, Weight(0.f)
		{
		}

		// Comparison Operator for Sorting
		struct FCompareBoneIndex
		{
			inline bool operator()(const FOrientationWarpingSpineBoneData& A, const FOrientationWarpingSpineBoneData& B) const
			{
				return A.BoneIndex < B.BoneIndex;
			}
		};
	};

	struct FOrientationWarpingFootData
	{
		TArray<FCompactPoseBoneIndex> IKFootBoneIndexArray;
		FCompactPoseBoneIndex IKFootRootBoneIndex;

		FOrientationWarpingFootData()
			: IKFootBoneIndexArray()
			, IKFootRootBoneIndex(INDEX_NONE)
		{
		}
	};

	// Computed spine bone indices and alpha weights for the specified spine definition 
	TArray<FOrientationWarpingSpineBoneData> SpineBoneDataArray;

	// Computed IK bone indices for the specified foot definitions 
	FOrientationWarpingFootData IKFootData;
	
	// Internal current frame root motion delta direction
	FVector RootMotionDeltaDirection = FVector::ZeroVector;

	// Internal current frame root motion delta angle
	FQuat RootMotionDeltaRotation = FQuat::Identity;

	// Target for counter compenstate, we keep the target so we can smoothly interp.
	float CounterCompensateTargetAngleRad = 0.0f;

#if ENABLE_ANIM_DEBUG || ENABLE_VISUAL_LOG
	// Store prediction in debug only so that they can persist across pauses
	FVector FutureRootMotionDeltaDirection = FVector::ZeroVector;
	bool bUsedFutureRootMotion = false;

	// Stored from blend message, used to match with rewind debugger
	FColor DebugColor = FColor::Black;
#endif

	// Internal orientation warping angle
	float ActualOrientationAngleRad = 0.f;
	float BlendWeight = 0.0f;

	FGraphTraversalCounter UpdateCounter;
	bool bIsFirstUpdate = false;
	void Reset(const FAnimationBaseContext& Context);

#if WITH_EDITORONLY_DATA
	// Whether we found a root motion delta attribute in the attribute stream on graph driven mode
	bool bFoundRootMotionAttribute = false;
#endif
};

#undef UE_API
