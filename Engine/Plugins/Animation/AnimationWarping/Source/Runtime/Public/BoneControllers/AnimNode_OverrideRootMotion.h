// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoneControllers/BoneControllerTypes.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "AnimNode_OverrideRootMotion.generated.h"

#define UE_API ANIMATIONWARPINGRUNTIME_API

struct FAnimationInitializeContext;
struct FComponentSpacePoseContext;
struct FNodeDebugData;

USTRUCT(BlueprintInternalUseOnly, Experimental)
struct FAnimNode_OverrideRootMotion : public FAnimNode_Base
{
	GENERATED_BODY();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Links, meta = (DisplayPriority = 0))
	FPoseLink Source;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = Evaluation, meta = (FoldProperty, PinShownByDefault))
	float Alpha = 1.f;

	UPROPERTY(EditAnywhere, Category = Evaluation, meta = (FoldProperty, PinShownByDefault))
	FVector OverrideVelocity = FVector(0,0,0);

	//todo: rotation override support
#endif

public:
	// FAnimNode_Base interface
	UE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	UE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	UE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	UE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	UE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	// Folded property accesors
	UE_API float GetAlpha() const;
	UE_API const FVector& GetOverrideVelocity() const;

private:

	// Internal cached anim instance proxy
	FAnimInstanceProxy* AnimInstanceProxy = nullptr;

	float DeltaTime = 0.f;
};

#undef UE_API
