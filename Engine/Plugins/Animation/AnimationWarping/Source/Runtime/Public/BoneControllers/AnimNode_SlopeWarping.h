// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "Engine/SpringInterpolator.h"
#include "AnimNode_SlopeWarping.generated.h"

#define UE_API ANIMATIONWARPINGRUNTIME_API

struct FAnimationInitializeContext;
struct FComponentSpacePoseContext;
struct FNodeDebugData;

/** Per foot definitions */
USTRUCT()
struct FSlopeWarpingFootDefinition
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Settings")
	FBoneReference IKFootBone;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FBoneReference FKFootBone;

	UPROPERTY(EditAnywhere, Category = "Settings")
	int32 NumBonesInLimb;

	UPROPERTY(EditAnywhere, Category = "Settings")
	float FootSize;

	FSlopeWarpingFootDefinition()
		: NumBonesInLimb(2)
		, FootSize(4.f)
	{}
};

/** Runtime foot data after validation, we guarantee these bones to exist */
USTRUCT()
struct FSlopeWarpingFootData
{
	GENERATED_USTRUCT_BODY()

public:
	FCompactPoseBoneIndex IKFootBoneIndex;
	FCompactPoseBoneIndex FKFootBoneIndex;
	FCompactPoseBoneIndex HipBoneIndex;

	FTransform IKFootTransform;

	FVector HitLocation;
	FVector HitNormal;

	FSlopeWarpingFootDefinition* FootDefinitionPtr;

public:
	FSlopeWarpingFootData()
		: IKFootBoneIndex(INDEX_NONE)
		, FKFootBoneIndex(INDEX_NONE)
		, HipBoneIndex(INDEX_NONE)
		, FootDefinitionPtr(nullptr)
	{}
};

class USkeletalMeshComponent;
class UCharacterMovementComponent;

USTRUCT(Experimental)
struct FAnimNode_SlopeWarping : public FAnimNode_SkeletalControlBase
{
	GENERATED_USTRUCT_BODY()

	UE_API FAnimNode_SlopeWarping();

	USkeletalMeshComponent* MySkelMeshComponent;
	UCharacterMovementComponent* MyMovementComponent;
	FAnimInstanceProxy* MyAnimInstanceProxy;

	/** IKFoot Root Bone. **/
	UPROPERTY(EditAnywhere, Category = "Settings")
	FBoneReference IKFootRootBone;

	/** Pelvis Bone. **/
	UPROPERTY(EditAnywhere, Category = "Settings")
	FBoneReference PelvisBone;

	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<FSlopeWarpingFootDefinition> FeetDefinitions;

	UPROPERTY(Transient)
	TArray<FSlopeWarpingFootData> FeetData;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FVectorRK4SpringInterpolator PelvisOffsetInterpolator;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (PinHiddenByDefault))
	FVector GravityDir;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (PinHiddenByDefault))
	FVector CustomFloorOffset;

	UPROPERTY(Transient)
	float CachedDeltaTime;

	UPROPERTY(Transient)
	FVector TargetFloorNormalWorldSpace;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FVectorRK4SpringInterpolator FloorNormalInterpolator;

	UPROPERTY(Transient)
	FVector TargetFloorOffsetLocalSpace;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FVectorRK4SpringInterpolator FloorOffsetInterpolator;

	UPROPERTY(EditAnywhere, Category = "Settings")
	float MaxStepHeight;

	UPROPERTY(EditAnywhere, Category = "Settings")
	uint32 bKeepMeshInsideOfCapsule : 1;

	UPROPERTY(EditAnywhere, Category = "Settings")
	uint32 bPullPelvisDown : 1;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (PinHiddenByDefault))
	uint32 bUseCustomFloorOffset : 1;

	UPROPERTY(Transient)
	uint32 bWasOnGround : 1;

	UPROPERTY(Transient)
	uint32 bShowDebug : 1;

	UPROPERTY(Transient)
	uint32 bFloorSmoothingInitialized : 1;

	UPROPERTY(Transient)
	FVector ActorLocation;

protected:
	UPROPERTY(Transient)
	FVector GravityDirCompSpace;

public:

	UE_API FVector GetFloorNormalFromMovementComponent() const;
	UE_API void SetWorldSpaceTargetFloorInfoFromMovementComponent();

	UE_API void GetSmoothedFloorInfo(FCSPose<FCompactPose>& MeshBases, FVector& OutFloorNormal, FVector& OutFloorOffset);

	// FAnimNode_Base interface
	UE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	UE_API virtual void UpdateInternal(const FAnimationUpdateContext& Context) override;
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	UE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	UE_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	UE_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

private:
	// FAnimNode_SkeletalControlBase interface
	UE_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface
};

#undef UE_API
