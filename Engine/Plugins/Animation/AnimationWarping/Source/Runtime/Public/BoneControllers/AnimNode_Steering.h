// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoneControllers/BoneControllerTypes.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "Kismet/KismetMathLibrary.h"
#include "AnimNode_Steering.generated.h"

#define UE_API ANIMATIONWARPINGRUNTIME_API

struct FAnimationInitializeContext;
struct FComponentSpacePoseContext;
struct FNodeDebugData;
class UAnimationAsset;

// Add procedural delta to the root motion attribute 
// This is done via 2 techniques:
//  1) Scaling the root motion on an animation
//  2) Adding additional correction to root motion after accounting for the anticipated (potentially scaled) root motion
// The effects of 1) and 2) combine
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_Steering : public FAnimNode_SkeletalControlBase
{
	GENERATED_BODY()

	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FAnimNode_Steering() = default;
	~FAnimNode_Steering() = default;
	FAnimNode_Steering(const FAnimNode_Steering&) = default;
	FAnimNode_Steering(FAnimNode_Steering&&) = default;
	FAnimNode_Steering& operator=(const FAnimNode_Steering&) = default;
	FAnimNode_Steering& operator=(FAnimNode_Steering&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// The Orientation to steer towards
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Evaluation, meta=(PinShownByDefault))
	FQuat TargetOrientation = FQuat::Identity;

	// True if input animation is mirrored
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Filtering, meta = (PinShownByDefault))
	bool bMirrored = false;
	
	// The number of seconds in the future before we should roughly attempt reach the TargetOrientation via additive correction
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Evaluation, meta=(PinShownByDefault))
	float ProceduralTargetTime = 0.2f;
	
	// Deprected old/unused parameter, to avoid breaking data
	UE_DEPRECATED(5.6, "Use Procedural target time for the correction time scale and AnimatedTargetTime for the look ahead time on the animation")
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use Procedural target time for the correction time scale and AnimatedTargetTime for the look ahead time on the animation"))
	float TargetTime_DEPRECATED = 0.2f;
	
	// The number of seconds in the future to sample the root motion to know how much this animation is expected to turn
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Evaluation, meta=(PinShownByDefault))
	float AnimatedTargetTime = 2.0f;

	// The minimum amount of root motion required to enable root motion scaling.
	// The root motion is measured from current time to AnimatedTargetTime
	UPROPERTY(EditAnywhere, DisplayName=RootMotionAngleThreshold, Category=Evaluation)
	float RootMotionThreshold = 1.0f;

	// below this movement speed (based on the root motion in the animation) disable steering completely (both scaling and additive)
	UPROPERTY(EditAnywhere, Category=Evaluation)
	float DisableSteeringBelowSpeed = 1.0f;

	// below this movement speed (based on the root motion in the animation) disable steering coming from the additive spring based correction
	UPROPERTY(EditAnywhere, Category = Evaluation)
	float DisableAdditiveBelowSpeed = -1.0f;

	// Will clamp the scaling ratio applied to above this threshold. Any error remaining will be compensated for linearly (using ProceduralTargetTime)
	UPROPERTY(EditAnywhere, Category = Evaluation)
	float MinScaleRatio = 0.5f;

	// Will clamp the scaling ratio applied to below this threshold. Any error remaining will be compensated for linearly (using ProceduralTargetTime)
	UPROPERTY(EditAnywhere, Category = Evaluation)
	float MaxScaleRatio = 1.5f;

	// If bMirrored is set, MirrorDataTable will be used for mirroring the CurrentAnimAsset during prediction
	UPROPERTY(EditAnywhere, Transient, BlueprintReadWrite, Category = Evaluation, meta = (PinShownByDefault))
	TObjectPtr<UMirrorDataTable> MirrorDataTable;
	
	// Animation Asset for incorporating root motion data. If CurrentAnimAsset is set, and the animation has root motion rotation within the TargetTime, then those rotations will be scaled to reach the TargetOrientation
	UPROPERTY(EditAnywhere, Transient, BlueprintReadWrite, Category=Evaluation, meta=(PinShownByDefault))
	TObjectPtr<UAnimationAsset> CurrentAnimAsset;
	
	// Current playback time in seconds of the CurrentAnimAsset
	UPROPERTY(EditAnywhere, Transient, BlueprintReadWrite, Category=Evaluation, meta=(PinShownByDefault))
	float CurrentAnimAssetTime = 0.f;

	// FAnimNodeBase interface
	UE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	UE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNodeBase interface
	
	// FAnimNode_SkeletalControlBase interface
	UE_API virtual void UpdateInternal(const FAnimationUpdateContext& Context) override;
	UE_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override { return true; }
	// End of FAnimNode_SkeletalControlBase interface

private:

	FVector AngularVelocity = FVector::ZeroVector;
	
	FTransform RootBoneTransform;
};

#undef UE_API
