// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoneControllers/BoneControllerTypes.h"
#include "BoneControllers/BoneControllerSolvers.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "AnimNode_StrideWarping.generated.h"

#define UE_API ANIMATIONWARPINGRUNTIME_API

struct FAnimationInitializeContext;
struct FComponentSpacePoseContext;
struct FNodeDebugData;

// Foot definition specifying the IK/FK foot bones and Thigh bone
USTRUCT(BlueprintInternalUseOnly)
struct FStrideWarpingFootDefinition
{
	GENERATED_BODY()

	// IK driven foot bone
	UPROPERTY(EditAnywhere, Category=Settings, meta=(DisplayName="IK Foot Bone"))
	FBoneReference IKFootBone;

	// FK driven foot bone
	UPROPERTY(EditAnywhere, Category=Settings, meta=(DisplayName="FK Foot Bone"))
	FBoneReference FKFootBone;

	// Thigh bone, representing the end of the leg chain prior to reaching the Pelvis Bone 
	UPROPERTY(EditAnywhere, Category=Settings)
	FBoneReference ThighBone;
};

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_StrideWarping : public FAnimNode_SkeletalControlBase
{
	GENERATED_BODY()

	// Stride warping evaluation mode (Graph or Manual)
	UPROPERTY(EditAnywhere, Category=Evaluation)
	EWarpingEvaluationMode Mode = EWarpingEvaluationMode::Manual;

	// Component-space stride direction
	// Example: A value of <1,0,0> will warp the leg stride along the Forward Vector
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Evaluation, meta=(PinShownByDefault))
	FVector StrideDirection = FVector::ForwardVector;

	// Stride scale, specifying the amount of warping applied to the foot definitions
	// Example: A value of 0.5 will decrease the effective leg stride by half, while a value of 2.0 will double it
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Evaluation, meta=(ClampMin="0.0", PinShownByDefault))
	float StrideScale = 1.f;

	// Locomotion speed, specifying the current speed of the character
	// This will be used in the following equation for computing the stride scale: [StrideScale = (LocomotionSpeed / RootMotionSpeed)]
	// Note: This speed should be relative to the delta time of the animation graph
	// 
	// Stride scale is a value specifying the amount of warping applied to the IK foot definitions
	// Example: A value of 0.5 will decrease the effective leg stride by half, while a value of 2.0 will double it
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Evaluation, meta=(ClampMin="0.0", PinShownByDefault))
	float LocomotionSpeed = 0.f;

	// Minimum root motion speed required to apply stride warping
	// This is useful to prevent unnatural strides when the animation has a portion with no root motion (i.e starts/stops)
	// When this value is greater than 0, it's recommended to enable interpolation in StrideScaleModifier
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Evaluation, meta=(ClampMin="0.0001", PinHiddenByDefault))
	float MinRootMotionSpeedThreshold = 10.0f;

	// Pevlis Bone definition
	UPROPERTY(EditAnywhere, Category=Settings)
	FBoneReference PelvisBone;

	// IK Foot Root Bone definition
	UPROPERTY(EditAnywhere, Category=Settings, meta=(DisplayName="IK Foot Root Bone"))
	FBoneReference IKFootRootBone;

	// Foot definitions specifying the IK, FK, and Thigh bone
	UPROPERTY(EditAnywhere, Category=Settings)
	TArray<FStrideWarpingFootDefinition> FootDefinitions;

	// Modifies the final stride scale value by optionally clamping and/or interpolating
	UPROPERTY(EditAnywhere, Category=Settings, meta=(PinHiddenByDefault))
	FInputClampConstants StrideScaleModifier;

	// Floor normal direction, this value will internally convert into a corresponding Component-space representation prior to warping
	// Default: World Space, Up Vector: <0,0,1>
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Advanced, meta=(PinHiddenByDefault))
	FWarpingVectorValue FloorNormalDirection = { EWarpingVectorMode::WorldSpaceVector, FVector::UpVector };

	// Gravity direction, this value will internally convert into a corresponding Component-space representation prior to warping
	// Default: World Space, Down Vector: <0,0,-1>
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Advanced, meta=(PinHiddenByDefault))
	FWarpingVectorValue GravityDirection = { EWarpingVectorMode::WorldSpaceVector, FVector::DownVector };

	// Solver for controlling how much the pelvis is "pulled down" towards the IK/FK foot definitions during leg limb extension
	UPROPERTY(EditAnywhere, Category=Advanced, meta=(DisplayName="Pelvis IK Foot Solver"))
	FIKFootPelvisPullDownSolver PelvisIKFootSolver;

	// Orients the specified (Manual) or computed (Graph) stride direction by the floor normal
	UPROPERTY(EditAnywhere, Category=Advanced)
	bool bOrientStrideDirectionUsingFloorNormal = true;

	// Include warping adjustment to the FK thigh bones alongside the IK/FK foot definitions
	// This is used to help preserve the original overall leg shape
	UPROPERTY(EditAnywhere, Category=Advanced, meta=(DisplayName="Compensate IK Using FK Thigh Rotation"))
	bool bCompensateIKUsingFKThighRotation = true;

	// Clamps the IK foot warping to prevent over-extension relative to the overall FK leg
	UPROPERTY(EditAnywhere, Category=Advanced, meta=(DisplayName="Clamp IK Using FK Limits", EditCondition="bCompensateIKUsingFKThighRotation"))
	bool bClampIKUsingFKLimits = true;

	// Do not execute stride warping if animation data has no root motion
	UPROPERTY(EditAnywhere, Category=Advanced)
	bool bDisableIfMissingRootMotion = true;

#if WITH_EDITORONLY_DATA
	// Scale all debug drawing visualization by a factor
	UPROPERTY(EditAnywhere, Category=Debug, meta=(ClampMin="0.0"))
	float DebugDrawScale = 1.f;

	// Enable/Disable stride warping debug drawing
	UPROPERTY(EditAnywhere, Category=Debug)
	bool bEnableDebugDraw = false;

	// Enable/Disable IK foot location debug drawing prior to warping
	UPROPERTY(EditAnywhere, Category=Debug)
	bool bDebugDrawIKFootOrigin = false;

	// Enable/Disable IK foot location debug drawing following initial foot adjustment
	UPROPERTY(EditAnywhere, Category=Debug)
	bool bDebugDrawIKFootAdjustment = false;

	// Enable/Disable pelvis debug drawing following adjustment
	UPROPERTY(EditAnywhere, Category=Debug)
	bool bDebugDrawPelvisAdjustment = false;

	// Enable/Disable thigh debug drawing following adjustment
	UPROPERTY(EditAnywhere, Category=Debug)
	bool bDebugDrawThighAdjustment = false;

	// Enable/Disable IK foot location debug drawing following all adsjustments (Final warped result)
	UPROPERTY(EditAnywhere, Category=Debug)
	bool bDebugDrawIKFootFinal = false;
#endif

public:
	// FAnimNode_Base interface
	UE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	UE_API virtual void UpdateInternal(const FAnimationUpdateContext& Context) override;
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	UE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	UE_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	UE_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

private:
	// FAnimNode_SkeletalControlBase interface
	UE_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

	struct FStrideWarpingFootData
	{
		FCompactPoseBoneIndex IKFootBoneIndex;
		FCompactPoseBoneIndex FKFootBoneIndex;
		FCompactPoseBoneIndex ThighBoneIndex;
		FTransform IKFootBoneTransform;

		FStrideWarpingFootData()
			: IKFootBoneIndex(INDEX_NONE)
			, FKFootBoneIndex(INDEX_NONE)
			, ThighBoneIndex(INDEX_NONE)
			, IKFootBoneTransform(FTransform::Identity)
		{
		}
	};

	// Internal cached anim instance proxy
	FAnimInstanceProxy* AnimInstanceProxy = nullptr;

	// Computed IK, FK, Thigh bone indices for the specified foot definitions
	TArray<FStrideWarpingFootData> FootData;

	// Internal cached stride scale modifier state
	FInputClampState StrideScaleModifierState;

	// Internal stride direction
	FVector ActualStrideDirection = FVector::ForwardVector;

	// Internal stride scale
	float ActualStrideScale = 1.f;

	// Internal cached delta time used for interpolators
	float CachedDeltaTime = 0.f;

#if WITH_EDITORONLY_DATA
	// Internal cached debug root motion delta translation
	FVector CachedRootMotionDeltaTranslation = FVector::ZeroVector;
	
	// Internal cached debug root motion speed
	float CachedRootMotionDeltaSpeed = 0.f;

	// Whether we found a root motion delta attribute in the attribute stream on graph driven mode
	bool bFoundRootMotionAttribute = false;
#endif
};

#undef UE_API
