// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimNodeBase.h"
#include "AnimNode_WarpTest.generated.h"

#define UE_API ANIMATIONWARPINGRUNTIME_API

USTRUCT(Experimental, BlueprintInternalUseOnly)
struct FAnimNode_WarpTest : public FAnimNode_Base
{
	GENERATED_BODY();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Links, meta = (DisplayPriority = 0))
	FPoseLink Source;

	// the node will warp the character looping between Transforms[i] choosing the next one every SecondsToWait
	UPROPERTY(EditAnywhere, Category = Evaluation, meta = (PinShownByDefault))
	TArray<FTransform> Transforms;

	// every SecondsToWait we warp to the next Transforms[i]
	UPROPERTY(EditAnywhere, Category = Evaluation, meta = (PinShownByDefault))
	float SecondsToWait = 1.f;

public:
	// FAnimNode_Base interface
	UE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	UE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	UE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	UE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	// End of FAnimNode_Base interface

	// ComponentTransform represents the previous frame component transform
	FTransform ComponentTransform = FTransform::Identity;
	float CurrentTime = 0.f;
	int32 CurrentTransformIndex = 0;
};

#undef UE_API
